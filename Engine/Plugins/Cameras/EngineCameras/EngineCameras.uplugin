{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Engine Cameras",
	"Description" : "Default engine camera shakes and animations",
	"Category" : "Cameras",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"Installed" : false,
	"Plugins":
	[
		{
			"Name": "TemplateSequence",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "EngineCameras",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		}
	]
}

