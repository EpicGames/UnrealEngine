// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class EngineCameras : ModuleRules
{
	public EngineCameras(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePaths.Add(Path.Combine(ModuleDirectory, "Legacy"));

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"CinematicCamera",
				"Core",
				"CoreUObject",
				"Engine",
				"HeadMountedDisplay",
				"MovieScene",
				"MovieSceneTracks",
				"TemplateSequence",
				"TraceLog"
			}
		);
	}
}
