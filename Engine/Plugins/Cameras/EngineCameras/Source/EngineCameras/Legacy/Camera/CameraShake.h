// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Deprecated

#include "CoreTypes.h"

UE_DEPRECATED_HEADER(5.5, "Use LegacyCameraShake.h instead of CameraShake.h")

#include "LegacyCameraShake.h"
