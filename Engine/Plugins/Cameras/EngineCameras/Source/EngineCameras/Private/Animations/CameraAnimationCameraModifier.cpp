// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraAnimationCameraModifier.h"

#include "Camera/CameraAnimationHelper.h"
#include "CameraAnimationSequence.h"
#include "CameraAnimationSequencePlayer.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "EngineCamerasModule.h"
#include "GameFramework/PlayerController.h"
#include "Kismet/GameplayStatics.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraAnimationCameraModifier)

DECLARE_CYCLE_STAT(TEXT("Camera Animation Eval"), CameraAnimationEval_Total, STATGROUP_CameraAnimation);

TAutoConsoleVariable<bool> GCameraAnimationLegacyPostProcessBlending(
	TEXT("r.CameraAnimation.LegacyPostProcessBlending"),
	true,
	TEXT("Blend camera animation post process settings under the main camera instead of over it"));

FCameraAnimationHandle FCameraAnimationHandle::Invalid(MAX_uint16, 0);

FActiveCameraAnimationInfo::FActiveCameraAnimationInfo()
	: Sequence(nullptr)
	, Handle(FCameraAnimationHandle::Invalid)
	, Player(nullptr)
	, CameraStandIn(nullptr)
	, EaseInCurrentTime(0)
	, EaseOutCurrentTime(0)
	, bIsEasingIn(false)
	, bIsEasingOut(false)
{
}

bool FActiveCameraAnimationInfo::IsValid() const
{
	return Sequence != nullptr;
}

bool FActiveCameraAnimationInfo::HasValidPlayer() const
{
	return Player && Player->IsValid();
}

namespace CameraAnimationCameraModifierImpl
{
	float Sinusoidal(float InTime)
	{
		return FMath::Sin(.5f*PI*InTime);
	}
	float Power(float InTime, float Power)
	{
		return FMath::Pow(InTime, Power);
	}
	float Exponential(float InTime)
	{
		return FMath::Pow(2, 10*(InTime - 1.f));
	}
	float Circular(float InTime)
	{
		return 1.f-FMath::Sqrt(1-InTime*InTime);
	}
}

float UCameraAnimationCameraModifier::EvaluateEasing(ECameraAnimationEasingType EasingType, float Interp)
{
	using namespace CameraAnimationCameraModifierImpl;
	
	switch (EasingType)
	{
	case ECameraAnimationEasingType::Sinusoidal: 	return Sinusoidal(Interp);
	case ECameraAnimationEasingType::Quadratic: 	return Power(Interp,2);
	case ECameraAnimationEasingType::Cubic:			return Power(Interp,3);
	case ECameraAnimationEasingType::Quartic:	 	return Power(Interp,4);
	case ECameraAnimationEasingType::Quintic:	 	return Power(Interp,5);
	case ECameraAnimationEasingType::Exponential: 	return Exponential(Interp);
	case ECameraAnimationEasingType::Circular:	 	return Circular(Interp);
	case ECameraAnimationEasingType::Linear:
	default:
		return Interp;
	}
}

UCameraAnimationCameraModifier::UCameraAnimationCameraModifier(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, NextInstanceSerialNumber(1)
{
}

FCameraAnimationHandle UCameraAnimationCameraModifier::PlayCameraAnimation(UCameraAnimationSequence* Sequence, FCameraAnimationParams Params)
{
	if (!ensure(Sequence))
	{
		return FCameraAnimationHandle::Invalid;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("PlayCameraAnimation %s"), *Sequence->GetFName().ToString()));

	int32 NewIndex = FindInactiveCameraAnimation();
	check(NewIndex < MAX_uint16);

	const uint16 InstanceSerial = NextInstanceSerialNumber++;
	FCameraAnimationHandle InstanceHandle { (uint16)NewIndex, InstanceSerial };

	FActiveCameraAnimationInfo& NewCameraAnimation = ActiveAnimations[NewIndex];
	NewCameraAnimation.Sequence = Sequence;
	NewCameraAnimation.Params = Params;
	NewCameraAnimation.Handle = InstanceHandle;

	const FName PlayerName = MakeUniqueObjectName(this, UCameraAnimationSequencePlayer::StaticClass(), TEXT("CameraAnimationPlayer"));
	NewCameraAnimation.Player = NewObject<UCameraAnimationSequencePlayer>(this, PlayerName);
	const FName CameraStandInName = MakeUniqueObjectName(this, UCameraAnimationSequenceCameraStandIn::StaticClass(), TEXT("CameraStandIn"));
	NewCameraAnimation.CameraStandIn = NewObject<UCameraAnimationSequenceCameraStandIn>(this, CameraStandInName);

	// Start easing in immediately if there's any defined.
	NewCameraAnimation.bIsEasingIn = (Params.EaseInDuration > 0.f);

	// Initialize our stand-in object.
	NewCameraAnimation.CameraStandIn->Initialize(Sequence);
	
	// Make the player always use our stand-in object whenever a sequence wants to spawn or possess an object.
	NewCameraAnimation.Player->SetBoundObjectOverride(NewCameraAnimation.CameraStandIn);

	// Initialize it and start playing.
	NewCameraAnimation.Player->Initialize(Sequence, Params.StartOffset);
	NewCameraAnimation.Player->Play(Params.bLoop, Params.bRandomStartTime);

	return InstanceHandle;
}

bool UCameraAnimationCameraModifier::IsCameraAnimationActive(const FCameraAnimationHandle& Handle) const
{
	const FActiveCameraAnimationInfo* CameraAnimation = GetActiveCameraAnimation(Handle);
	if (CameraAnimation)
	{
		return CameraAnimation->Player->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing;
	}
	return false;
}

void UCameraAnimationCameraModifier::StopCameraAnimation(const FCameraAnimationHandle& Handle, bool bImmediate)
{
	FActiveCameraAnimationInfo* CameraAnimation = GetActiveCameraAnimation(Handle);
	if (CameraAnimation)
	{
		check(CameraAnimation->Sequence && CameraAnimation->Player);

		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("StopCameraAnimation %s"), *CameraAnimation->Sequence->GetFName().ToString()));

		if (bImmediate || CameraAnimation->Params.EaseOutDuration == 0.f)
		{
			CameraAnimation->Player->Stop();
			ActiveAnimations[CameraAnimation->Handle.InstanceID] = FActiveCameraAnimationInfo();
		}
		else if (!CameraAnimation->bIsEasingOut)
		{
			CameraAnimation->bIsEasingOut = true;
			CameraAnimation->EaseOutCurrentTime = 0.f;
		}
	}
}

void UCameraAnimationCameraModifier::StopAllCameraAnimationsOf(UCameraAnimationSequence* Sequence, bool bImmediate)
{
	for (FActiveCameraAnimationInfo& CameraAnimation : ActiveAnimations)
	{
		if (CameraAnimation.Sequence == Sequence)
		{
			StopCameraAnimation(CameraAnimation.Handle, bImmediate);
		}
	}
}

void UCameraAnimationCameraModifier::StopAllCameraAnimations(bool bImmediate)
{
	for (FActiveCameraAnimationInfo& CameraAnimation : ActiveAnimations)
	{
		StopCameraAnimation(CameraAnimation.Handle, bImmediate);
	}
}

UCameraAnimationCameraModifier* UCameraAnimationCameraModifier::GetCameraAnimationCameraModifier(const UObject* WorldContextObject, int32 PlayerIndex)
{
	APlayerController* PlayerController = UGameplayStatics::GetPlayerController(WorldContextObject, PlayerIndex);
	return GetCameraAnimationCameraModifierFromPlayerController(PlayerController);
}

UCameraAnimationCameraModifier* UCameraAnimationCameraModifier::GetCameraAnimationCameraModifierFromID(const UObject* WorldContextObject, int32 ControllerID)
{
	APlayerController* PlayerController = UGameplayStatics::GetPlayerControllerFromID(WorldContextObject, ControllerID);
	return GetCameraAnimationCameraModifierFromPlayerController(PlayerController);
}

UCameraAnimationCameraModifier* UCameraAnimationCameraModifier::GetCameraAnimationCameraModifierFromPlayerController(const APlayerController* PlayerController)
{
	if (PlayerController)
	{
		UCameraModifier* CameraModifier = PlayerController->PlayerCameraManager->FindCameraModifierByClass(UCameraAnimationCameraModifier::StaticClass());
		return CastChecked<UCameraAnimationCameraModifier>(CameraModifier, ECastCheckedType::NullAllowed);
	}
	return nullptr;
}

bool UCameraAnimationCameraModifier::ModifyCamera(float DeltaTime, struct FMinimalViewInfo& InOutPOV)
{
	Super::ModifyCamera(DeltaTime, InOutPOV);
	TickAllAnimations(DeltaTime, InOutPOV);
	return false;
}

void UCameraAnimationCameraModifier::DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
	Canvas->SetDrawColor(FColor::Yellow);
	UFont* DrawFont = GEngine->GetSmallFont();

	int Indentation = 1;
	int LineNumber = FMath::CeilToInt(YPos / YL);

	Canvas->DrawText(DrawFont, FString::Printf(TEXT("Modifier_CameraAnimationSequence %s, Alpha:%f"), *GetNameSafe(this), Alpha), Indentation * YL, (LineNumber++) * YL);

	Indentation = 2;
	for (int Index = 0; Index < ActiveAnimations.Num(); Index++)
	{
		FActiveCameraAnimationInfo& ActiveAnimation = ActiveAnimations[Index];

		if (ActiveAnimation.IsValid())
		{
			const FFrameRate DisplayRate = ActiveAnimation.Player->GetInputRate();
			const FFrameTime DurationFrames = ActiveAnimation.Player->GetDuration();
			const FFrameTime CurrentPosition = ActiveAnimation.Player->GetCurrentPosition();
			const FFrameTime StartFrame = ActiveAnimation.Player->GetStartFrame();

			const float ElapsedTime = DisplayRate.AsSeconds(CurrentPosition - StartFrame);
			const float DurationSeconds = DisplayRate.AsSeconds(DurationFrames);

			const FString LoopString = ActiveAnimation.Params.bLoop ? TEXT(" Looping") : TEXT("");
			const FString EaseInString = ActiveAnimation.bIsEasingIn ? FString::Printf(TEXT(" Easing In: %f / %f"), ActiveAnimation.EaseInCurrentTime, ActiveAnimation.Params.EaseInDuration) : TEXT("");
			const FString EaseOutString = ActiveAnimation.bIsEasingOut ? FString::Printf(TEXT(" Easing Out: %f / %f"), ActiveAnimation.EaseOutCurrentTime, ActiveAnimation.Params.EaseOutDuration) : TEXT("");
			Canvas->DrawText(
					DrawFont,
					FString::Printf(
						TEXT("[%d] %s PlayRate: %f Duration: %f Elapsed: %f%s%s"),
						Index, *GetNameSafe(ActiveAnimation.Sequence), 
						ActiveAnimation.Params.PlayRate, DurationSeconds, ElapsedTime, *EaseInString, *EaseOutString),
					Indentation* YL, (LineNumber++)* YL);
		}
	}

	YPos = LineNumber * YL;

	Super::DisplayDebug(Canvas, DebugDisplay, YL, YPos);
}

int32 UCameraAnimationCameraModifier::FindInactiveCameraAnimation()
{
	for (int32 Index = 0; Index < ActiveAnimations.Num(); ++Index)
	{
		const FActiveCameraAnimationInfo& CameraAnimation(ActiveAnimations[Index]);
		if (!CameraAnimation.IsValid())
		{
			// Reset this entry to default.
			ActiveAnimations[Index] = FActiveCameraAnimationInfo();
			return Index;
		}
	}

	return ActiveAnimations.Emplace();
}

const FActiveCameraAnimationInfo* UCameraAnimationCameraModifier::GetActiveCameraAnimation(const FCameraAnimationHandle& Handle) const
{
	if (Handle.IsValid() && ActiveAnimations.IsValidIndex(Handle.InstanceID))
	{
		const FActiveCameraAnimationInfo& CameraAnimation = ActiveAnimations[Handle.InstanceID];
		if (CameraAnimation.Handle.InstanceSerial == Handle.InstanceSerial)
		{
			return &CameraAnimation;
		}
	}
	return nullptr;
}

FActiveCameraAnimationInfo* UCameraAnimationCameraModifier::GetActiveCameraAnimation(const FCameraAnimationHandle& Handle)
{
	if (Handle.IsValid() && ActiveAnimations.IsValidIndex(Handle.InstanceID))
	{
		FActiveCameraAnimationInfo& CameraAnimation = ActiveAnimations[Handle.InstanceID];
		if (CameraAnimation.Handle.InstanceSerial == Handle.InstanceSerial)
		{
			return &CameraAnimation;
		}
	}
	return nullptr;
}

void UCameraAnimationCameraModifier::DeactivateCameraAnimation(int32 Index)
{
	check(ActiveAnimations.IsValidIndex(Index));
	FActiveCameraAnimationInfo& CameraAnimation = ActiveAnimations[Index];

	check(CameraAnimation.Player);
	if (!ensure(CameraAnimation.Player->GetPlaybackStatus() == EMovieScenePlayerStatus::Stopped))
	{
		CameraAnimation.Player->Stop();
	}

	CameraAnimation = FActiveCameraAnimationInfo();
}

void UCameraAnimationCameraModifier::TickAllAnimations(float DeltaTime, FMinimalViewInfo& InOutPOV)
{
	for (FActiveCameraAnimationInfo& ActiveAnimation : ActiveAnimations)
	{
		if (ActiveAnimation.IsValid())
		{
			if (ActiveAnimation.HasValidPlayer())
			{
				TickAnimation(ActiveAnimation, DeltaTime, InOutPOV);

				if (ActiveAnimation.Player->GetPlaybackStatus() == EMovieScenePlayerStatus::Stopped)
				{
					DeactivateCameraAnimation(ActiveAnimation.Handle.InstanceID);
				}
			}
			else
			{
				// The animation's player might have become invalid due to things like server travel 
				// to a new level. The player's linker would have been destroyed with the old level.
				// In that case, we abort the camera animation.
				ActiveAnimation.Sequence = nullptr;
				ActiveAnimation.Player = nullptr;
			}
		}
	}
}

void UCameraAnimationCameraModifier::TickAnimation(FActiveCameraAnimationInfo& CameraAnimation, float DeltaTime, FMinimalViewInfo& InOutPOV)
{
	SCOPE_CYCLE_COUNTER(CameraAnimationEval_Total);

	check(CameraAnimation.Player);
	check(CameraAnimation.CameraStandIn);

	const FCameraAnimationParams Params = CameraAnimation.Params;
	UCameraAnimationSequencePlayer* Player = CameraAnimation.Player;
	UCameraAnimationSequenceCameraStandIn* CameraStandIn = CameraAnimation.CameraStandIn;

	const FFrameRate DisplayRate = Player->GetInputRate();
	const FFrameTime CurrentPosition = Player->GetCurrentPosition();

	const float ScaledDeltaTime = DeltaTime * Params.PlayRate;
	const FFrameTime NewPosition = CurrentPosition + ScaledDeltaTime * DisplayRate;

	// Advance any easing times.
	if (CameraAnimation.bIsEasingIn)
	{
		CameraAnimation.EaseInCurrentTime += DeltaTime;
	}
	if (CameraAnimation.bIsEasingOut)
	{
		CameraAnimation.EaseOutCurrentTime += DeltaTime;
	}

	// Start easing out if we're nearing the end.
	// We might already be easing out if we have been told to stop. In that case, don't reset
	// the ease out, just continue doing it.
	if (!Player->GetIsLooping())
	{
		const float ElapsedTime = DisplayRate.AsSeconds(NewPosition - Player->GetStartFrame());
		const float DurationTime = DisplayRate.AsSeconds(Player->GetDuration()) * Params.PlayRate;
		const float BlendOutStartTime = DurationTime - Params.EaseOutDuration;
		if (ElapsedTime > BlendOutStartTime && !CameraAnimation.bIsEasingOut)
		{
			CameraAnimation.bIsEasingOut = true;
			CameraAnimation.EaseOutCurrentTime = ElapsedTime - BlendOutStartTime;
		}
	}

	// Check if we're done easing in or out.
	bool bIsDoneEasingOut = false;
	if (CameraAnimation.bIsEasingIn)
	{
		if (CameraAnimation.EaseInCurrentTime > Params.EaseInDuration || Params.EaseInDuration == 0.f)
		{
			CameraAnimation.bIsEasingIn = false;
		}
	}
	if (CameraAnimation.bIsEasingOut)
	{
		if (CameraAnimation.EaseOutCurrentTime > Params.EaseOutDuration)
		{
			bIsDoneEasingOut = true;
		}
	}

	// Figure out the final easing weight.
	const float EasingInT = FMath::Clamp((CameraAnimation.EaseInCurrentTime / Params.EaseInDuration), 0.f, 1.f);
	const float EasingInWeight = CameraAnimation.bIsEasingIn ?
		EvaluateEasing(Params.EaseInType, EasingInT) : 1.f;

	const float EasingOutT = FMath::Clamp((1.f - CameraAnimation.EaseOutCurrentTime / Params.EaseOutDuration), 0.f, 1.f);
	const float EasingOutWeight = CameraAnimation.bIsEasingOut ?
		EvaluateEasing(Params.EaseOutType, EasingOutT) : 1.f;

	const float TotalEasingWeight = FMath::Min(EasingInWeight, EasingOutWeight);

	// We might be done playing. Normally the player will stop on its own, but there are other situation in which
	// the responsibility falls to this code:
	// - If the animation is looping and waiting for an explicit Stop() call on us.
	// - If there was a Stop() call with bImmediate=false to let an animation blend out.
	if (bIsDoneEasingOut || TotalEasingWeight <= 0.f)
	{
		Player->Stop();
		return;
	}

	UMovieSceneEntitySystemLinker* Linker = Player->GetEvaluationTemplate().GetEntitySystemLinker();
	CameraStandIn->Reset(InOutPOV, Linker);

	// Get the "unanimated" properties that need to be treated additively.
	const float OriginalFieldOfView = CameraStandIn->FieldOfView;

	// Update the sequence.
	Player->Update(NewPosition);

	// Recalculate properties that might be invalidated by other properties having been animated.
	CameraStandIn->RecalcDerivedData();

	// Grab the final animated (animated) values, figure out the delta, apply scale, and feed that into the result.
	// Transform is always treated as a local, additive value. The data better be good.
	const float Scale = Params.Scale * TotalEasingWeight;
	const FTransform AnimatedTransform = CameraStandIn->GetTransform();
	FVector AnimatedLocation = AnimatedTransform.GetLocation() * Scale;
	FRotator AnimatedRotation = AnimatedTransform.GetRotation().Rotator() * Scale;

	const bool bIsCameraLocal = (Params.PlaySpace == ECameraAnimationPlaySpace::CameraLocal);
	const FMatrix UserPlaySpaceMatrix = (Params.PlaySpace == ECameraAnimationPlaySpace::UserDefined) ? 
		FRotationMatrix(Params.UserPlaySpaceRot) : FRotationMatrix::Identity;
	const FCameraAnimationHelperOffset CameraOffset { AnimatedLocation, AnimatedRotation };
	if (bIsCameraLocal)
	{
		FCameraAnimationHelper::ApplyOffset(InOutPOV, CameraOffset, AnimatedLocation, AnimatedRotation);
	}
	else
	{
		FCameraAnimationHelper::ApplyOffset(UserPlaySpaceMatrix, InOutPOV, CameraOffset, AnimatedLocation, AnimatedRotation);
	}
	InOutPOV.Location = AnimatedLocation;
	InOutPOV.Rotation = AnimatedRotation;

	// FieldOfView follows the current camera's value every frame, so we can compute how much the animation is
	// changing it.
	const float AnimatedFieldOfView = CameraStandIn->FieldOfView;
	const float DeltaFieldOfView = AnimatedFieldOfView - OriginalFieldOfView;
	InOutPOV.FOV = OriginalFieldOfView + DeltaFieldOfView * Scale;

	// Add the post-process settings.
	if (CameraOwner != nullptr && CameraStandIn->PostProcessBlendWeight > 0.f)
	{
		const float TotalPostProcessBlendWeight = CameraStandIn->PostProcessBlendWeight * Scale;
		EViewTargetBlendOrder CameraShakeBlendOrder = GCameraAnimationLegacyPostProcessBlending.GetValueOnGameThread() ? VTBlendOrder_Base : VTBlendOrder_Override;
		CameraOwner->AddCachedPPBlend(CameraStandIn->PostProcessSettings, TotalPostProcessBlendWeight, CameraShakeBlendOrder);
	}
}

UCameraAnimationCameraModifier* UEngineCameraAnimationFunctionLibrary::Conv_CameraAnimationCameraModifier(APlayerCameraManager* PlayerCameraManager)
{
	return PlayerCameraManager ? Cast<UCameraAnimationCameraModifier>(PlayerCameraManager->FindCameraModifierByClass(UCameraAnimationCameraModifier::StaticClass())) : nullptr;
}

ECameraShakePlaySpace UEngineCameraAnimationFunctionLibrary::Conv_CameraShakePlaySpace(ECameraAnimationPlaySpace CameraAnimationPlaySpace)
{
	switch (CameraAnimationPlaySpace)
	{
	case ECameraAnimationPlaySpace::CameraLocal: return ECameraShakePlaySpace::CameraLocal;
	case ECameraAnimationPlaySpace::World: return ECameraShakePlaySpace::World;
	case ECameraAnimationPlaySpace::UserDefined: return ECameraShakePlaySpace::UserDefined;
	default: checkf(false, TEXT("Unsupported ECameraAnimationPlaySpace value")); return (ECameraShakePlaySpace)CameraAnimationPlaySpace;
	}
}

ECameraAnimationPlaySpace UEngineCameraAnimationFunctionLibrary::Conv_CameraAnimationPlaySpace(ECameraShakePlaySpace CameraShakePlaySpace)
{
	switch (CameraShakePlaySpace)
	{
	case ECameraShakePlaySpace::CameraLocal: return ECameraAnimationPlaySpace::CameraLocal;
	case ECameraShakePlaySpace::World: return ECameraAnimationPlaySpace::World;
	case ECameraShakePlaySpace::UserDefined: return ECameraAnimationPlaySpace::UserDefined;
	default: checkf(false, TEXT("Unsupported ECameraShakePlaySpace value")); return (ECameraAnimationPlaySpace)CameraShakePlaySpace;
	}
}

