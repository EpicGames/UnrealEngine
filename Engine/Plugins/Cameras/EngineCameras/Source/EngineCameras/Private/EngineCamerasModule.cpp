// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineCamerasModule.h"

#include "Camera/CameraModularFeature.h"
#include "CameraAnimationCameraModifier.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"

class FEngineCamerasModule : public IEngineCamerasModule
{
public:

	// IModuleInterface interface
	virtual void StartupModule() override
	{
		CameraModularFeature = MakeShared<FCameraModularFeature>();
		if (CameraModularFeature.IsValid())
		{
			IModularFeatures::Get().RegisterModularFeature(ICameraModularFeature::GetModularFeatureName(), CameraModularFeature.Get());
		}
	}

	virtual void ShutdownModule() override
	{
		if (CameraModularFeature.IsValid())
		{
			IModularFeatures::Get().UnregisterModularFeature(ICameraModularFeature::GetModularFeatureName(), CameraModularFeature.Get());
			CameraModularFeature = nullptr;
		}
	}

private:

	class FCameraModularFeature : public ICameraModularFeature
	{
		// ICameraModularFeature interface
		virtual void GetDefaultModifiers(TArray<TSubclassOf<UCameraModifier>>& ModifierClasses) const override
		{
			ModifierClasses.Add(UCameraAnimationCameraModifier::StaticClass());
		}
	};

	TSharedPtr<FCameraModularFeature> CameraModularFeature;
};

IMPLEMENT_MODULE(FEngineCamerasModule, EngineCameras);

