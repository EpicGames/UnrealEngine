// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineCamerasSubsystem.h"
#include "CameraAnimationCameraModifier.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EngineCamerasSubsystem)

#define LOCTEXT_NAMESPACE "EngineCamerasSubsystem"

UEngineCamerasSubsystem* UEngineCamerasSubsystem::GetEngineCamerasSubsystem(const UWorld* InWorld)
{
	if (InWorld)
	{
		return InWorld->GetSubsystem<UEngineCamerasSubsystem>();
	}

	return nullptr;
}

FCameraAnimationHandle UEngineCamerasSubsystem::PlayCameraAnimation(APlayerController* PlayerController, UCameraAnimationSequence* Sequence, FCameraAnimationParams Params)
{
	UCameraAnimationCameraModifier* CameraModifier = UCameraAnimationCameraModifier::GetCameraAnimationCameraModifierFromPlayerController(PlayerController);
	if (ensureMsgf(CameraModifier, TEXT("No camera modifier found on the player controller")))
	{
		return CameraModifier->PlayCameraAnimation(Sequence, Params);
	}
	FFrame::KismetExecutionMessage(TEXT("Can't play camera animation: no camera animation modifier found"), ELogVerbosity::Error);
	return FCameraAnimationHandle::Invalid;
}

bool UEngineCamerasSubsystem::IsCameraAnimationActive(APlayerController* PlayerController, const FCameraAnimationHandle& Handle) const
{
	UCameraAnimationCameraModifier* CameraModifier = UCameraAnimationCameraModifier::GetCameraAnimationCameraModifierFromPlayerController(PlayerController);
	if (CameraModifier)
	{
		return CameraModifier->IsCameraAnimationActive(Handle);
	}
	return false;
}

void UEngineCamerasSubsystem::StopCameraAnimation(APlayerController* PlayerController, const FCameraAnimationHandle& Handle, bool bImmediate)
{
	UCameraAnimationCameraModifier* CameraModifier = UCameraAnimationCameraModifier::GetCameraAnimationCameraModifierFromPlayerController(PlayerController);
	if (ensureMsgf(CameraModifier, TEXT("No camera modifier found on the player controller")))
	{
		CameraModifier->StopCameraAnimation(Handle, bImmediate);
		return;
	}
	FFrame::KismetExecutionMessage(TEXT("Can't stop camera animation: no camera animation modifier found"), ELogVerbosity::Error);
}

void UEngineCamerasSubsystem::StopAllCameraAnimationsOf(APlayerController* PlayerController, UCameraAnimationSequence* Sequence, bool bImmediate)
{
	UCameraAnimationCameraModifier* CameraModifier = UCameraAnimationCameraModifier::GetCameraAnimationCameraModifierFromPlayerController(PlayerController);
	if (ensureMsgf(CameraModifier, TEXT("No camera modifier found on the player controller")))
	{
		CameraModifier->StopAllCameraAnimationsOf(Sequence, bImmediate);
		return;
	}
	FFrame::KismetExecutionMessage(TEXT("Can't stop camera animations: no camera animation modifier found"), ELogVerbosity::Error);
}

void UEngineCamerasSubsystem::StopAllCameraAnimations(APlayerController* PlayerController, bool bImmediate)
{
	UCameraAnimationCameraModifier* CameraModifier = UCameraAnimationCameraModifier::GetCameraAnimationCameraModifierFromPlayerController(PlayerController);
	if (ensureMsgf(CameraModifier, TEXT("No camera modifier found on the player controller")))
	{
		CameraModifier->StopAllCameraAnimations(bImmediate);
		return;
	}
	FFrame::KismetExecutionMessage(TEXT("Can't stop all camera animation: no camera animation modifier found"), ELogVerbosity::Error);
}

#undef LOCTEXT_NAMESPACE


