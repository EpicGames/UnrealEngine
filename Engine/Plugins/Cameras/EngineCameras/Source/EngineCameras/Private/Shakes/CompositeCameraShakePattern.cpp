// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shakes/CompositeCameraShakePattern.h"
#include "Camera/PlayerCameraManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositeCameraShakePattern)

void UCompositeCameraShakePattern::GetShakePatternInfoImpl(FCameraShakeInfo& OutInfo) const
{
	// Set the duration to fixed at first. We'll change it to something different
	// if we encounter children with different duration types.
	// The duration value will be the maximum of all our children's durations.
	float Duration = 0.f;
	ECameraShakeDurationType DurationType = ECameraShakeDurationType::Fixed;

	for (UCameraShakePattern* Pattern : ChildPatterns)
	{
		if (Pattern != nullptr)
		{
			FCameraShakeInfo ChildInfo;
			Pattern->GetShakePatternInfo(ChildInfo);

			switch (ChildInfo.Duration.GetDurationType())
			{
				case ECameraShakeDurationType::Infinite:
					// If one of our children is infinite, we are infinite.
					Duration = 0.f;
					DurationType = ECameraShakeDurationType::Infinite;
					break;
				case ECameraShakeDurationType::Custom:
					// Change our type to custom, but include the child's duration hint
					// in our own hint.
					Duration = FMath::Max(Duration, ChildInfo.Duration.Get());
					DurationType = ECameraShakeDurationType::Custom;
					break;
				case ECameraShakeDurationType::Fixed:
					// Grow our fixed duration if necessary.
					Duration = FMath::Max(Duration, ChildInfo.Duration.Get());
					break;
			}

			if (DurationType == ECameraShakeDurationType::Infinite)
			{
				// Can't get any bigger than infinite.
				break;
			}
		}
	}

	OutInfo.Duration = FCameraShakeDuration(Duration, DurationType);
}

void UCompositeCameraShakePattern::StartShakePatternImpl(const FCameraShakePatternStartParams& Params)
{
	for (UCameraShakePattern* Pattern : ChildPatterns)
	{
		if (Pattern != nullptr)
		{
			// Start the child pattern.
			Pattern->StartShakePattern(Params);
		}
	}
}

void UCompositeCameraShakePattern::UpdateShakePatternImpl(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	UCameraShakeBase* ShakeInstance = GetShakeInstance();
	checkf(ShakeInstance, TEXT("Running a shake pattern without an outer shake instance"));

	FCameraShakePatternUpdateParams ChildParams(Params);

	for (UCameraShakePattern* Pattern : ChildPatterns)
	{
		if (Pattern != nullptr && !Pattern->IsFinished())
		{
			// Let the child pattern run on the current result, with its own blending weight.
			FCameraShakePatternUpdateResult ChildResult;
			Pattern->UpdateShakePattern(ChildParams, ChildResult);

			if (!Pattern->IsFinished())
			{
				// Apply this result and pass it on to the next child pattern.
				FCameraShakeApplyResultParams ApplyParams;
				ApplyParams.Scale = Params.GetTotalScale();
				ApplyParams.PlaySpace = ShakeInstance->GetPlaySpace();
				ApplyParams.UserPlaySpaceMatrix = ShakeInstance->GetUserPlaySpaceMatrix();
				ApplyParams.CameraManager = ShakeInstance->GetCameraManager();
				// This applies the current pattern's update to the parameters we'll pass
				// to the next one.
				UCameraShakeBase::ApplyResult(ApplyParams, ChildResult, ChildParams.POV);
			}
		}
	}

	// All our children patterns have applied their logic to our temp view, so we can put this in
	// our own result, marking it as absolute.
	OutResult.Location = ChildParams.POV.Location;
	OutResult.Rotation = ChildParams.POV.Rotation;
	OutResult.FOV = ChildParams.POV.FOV;
	OutResult.Flags = ECameraShakePatternUpdateResultFlags::ApplyAsAbsolute;
}

void UCompositeCameraShakePattern::ScrubShakePatternImpl(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	// This method is similar to the UpdateShakePatternImpl method above, but calls the scrub methods
	// instead of the update methods.

	UCameraShakeBase* ShakeInstance = GetShakeInstance();
	checkf(ShakeInstance, TEXT("Running a shake pattern without an outer shake instance"));

	FCameraShakePatternScrubParams ChildParams(Params);

	for (UCameraShakePattern* Pattern : ChildPatterns)
	{
		if (Pattern != nullptr) // Don't check for IsFinished here, we might scrub anywhere.
		{
			// Let the child pattern run on the current result, with its own blending weight.
			FCameraShakePatternUpdateResult ChildResult;
			Pattern->ScrubShakePattern(Params, ChildResult);

			if (!Pattern->IsFinished())
			{
				// Apply this result and pass it on to the next child pattern.
				FCameraShakeApplyResultParams ApplyParams;
				ApplyParams.Scale = Params.GetTotalScale();
				ApplyParams.PlaySpace = ShakeInstance->GetPlaySpace();
				ApplyParams.UserPlaySpaceMatrix = ShakeInstance->GetUserPlaySpaceMatrix();
				// This applies the current pattern's scrubbing to the parameters we'll pass
				// to the next one.
				UCameraShakeBase::ApplyResult(ApplyParams, ChildResult, ChildParams.POV);
			}
		}
	}

	// All our children patterns have applied their logic to our temp view, so we can put this in
	// our own result, marking it as absolute.
	OutResult.Location = ChildParams.POV.Location;
	OutResult.Rotation = ChildParams.POV.Rotation;
	OutResult.FOV = ChildParams.POV.FOV;
	OutResult.Flags = ECameraShakePatternUpdateResultFlags::ApplyAsAbsolute;
}

bool UCompositeCameraShakePattern::IsFinishedImpl() const
{
	// We're not finished if any of our child patterns is not finished.
	for (UCameraShakePattern* Pattern : ChildPatterns)
	{
		if (Pattern != nullptr && !Pattern->IsFinished())
		{
			return false;
		}
	}
	return true;
}

void UCompositeCameraShakePattern::StopShakePatternImpl(const FCameraShakePatternStopParams& Params)
{
	// Stop all our children.
	for (UCameraShakePattern* Pattern : ChildPatterns)
	{
		if (Pattern != nullptr)
		{
			Pattern->StopShakePattern(Params);
		}
	}
}

void UCompositeCameraShakePattern::TeardownShakePatternImpl()
{
	// Teardown are all our children.
	for (UCameraShakePattern* Pattern : ChildPatterns)
	{
		if (Pattern != nullptr)
		{
			Pattern->TeardownShakePattern();
		}
	}
}

