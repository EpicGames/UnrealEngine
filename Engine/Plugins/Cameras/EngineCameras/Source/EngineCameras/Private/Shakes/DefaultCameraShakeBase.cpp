// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shakes/DefaultCameraShakeBase.h"
#include "Shakes/PerlinNoiseCameraShakePattern.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DefaultCameraShakeBase)

UDefaultCameraShakeBase::UDefaultCameraShakeBase(const FObjectInitializer& ObjInit)
	: Super(ObjInit
				.SetDefaultSubobjectClass<UPerlinNoiseCameraShakePattern>(TEXT("RootShakePattern")))
{
}


