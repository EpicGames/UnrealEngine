// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shakes/LegacyCameraShake.h"
#include "CameraAnimationSequence.h"
#include "SequenceCameraShake.h"
#include "Camera/PlayerCameraManager.h"
#include "Engine/Engine.h"
#include "IXRTrackingSystem.h" // for IsHeadTrackingAllowed()
#include "MovieScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LegacyCameraShake)

DEFINE_LOG_CATEGORY_STATIC(LogLegacyCameraShake, Warning, All);

//////////////////////////////////////////////////////////////////////////
// FFOscillator

// static
float FFOscillator::UpdateOffset(FFOscillator const& Osc, float& CurrentOffset, float DeltaTime)
{
	// LWC_TODO: Perf pessimization
	double AsDouble = CurrentOffset;
	float Result = UpdateOffset(Osc, AsDouble, DeltaTime);
	CurrentOffset = (float)AsDouble;
	return Result;
}

// static
float FFOscillator::UpdateOffset(FFOscillator const& Osc, double& CurrentOffset, float DeltaTime)
{
	if (Osc.Amplitude != 0.f)
	{
		CurrentOffset += DeltaTime * Osc.Frequency;

		float WaveformSample;
		switch (Osc.Waveform)
		{
		case EOscillatorWaveform::SineWave:
		default:
			WaveformSample = FMath::Sin(CurrentOffset);
			break;

		case EOscillatorWaveform::PerlinNoise:
			WaveformSample = FMath::PerlinNoise1D(CurrentOffset);
			break;
		}

		return Osc.Amplitude * WaveformSample;
	}
	return 0.f;
}

// static
float FFOscillator::GetInitialOffset(FFOscillator const& Osc)
{
	return (Osc.InitialOffset == EOO_OffsetRandom)
		? FMath::FRand() * (2.f * PI)
		: 0.f;
}

// static
float FFOscillator::GetOffsetAtTime(FFOscillator const& Osc, float InitialOffset, float Time)
{
	return InitialOffset + (Time * Osc.Frequency);
}

//////////////////////////////////////////////////////////////////////////
// ULegacyCameraShake

ULegacyCameraShake::ULegacyCameraShake(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer
				.SetDefaultSubobjectClass<ULegacyCameraShakePattern>(TEXT("RootShakePattern")))
{
	AnimPlayRate = 1.0f;
	AnimScale = 1.0f;
	AnimBlendInTime = 0.2f;
	AnimBlendOutTime = 0.2f;
	OscillationBlendInTime = 0.1f;
	OscillationBlendOutTime = 0.2f;
}

void ULegacyCameraShake::DoStopShake(bool bImmediately)
{
	if (bImmediately)
	{
		// stop oscillation
		OscillatorTimeRemaining = 0.f;
	}
	else
	{
		// advance to the blend out time
		if (OscillatorTimeRemaining > 0.0f)
		{
			OscillatorTimeRemaining = FMath::Min(OscillatorTimeRemaining, OscillationBlendOutTime);
		}
		else
		{
			OscillatorTimeRemaining = OscillationBlendOutTime;
		}
	}
	
	if (SequenceShakePattern)
	{
		FCameraShakePatternStopParams StopParams;
		StopParams.bImmediately = bImmediately;
		SequenceShakePattern->StopShakePattern(StopParams);
	}

	UE_LOGF(LogLegacyCameraShake, Verbose, "ULegacyCameraShake::DoStopShake %ls", *GetNameSafe(this));

	ReceiveStopShake(bImmediately);
}

void ULegacyCameraShake::DoStartShake(const FCameraShakePatternStartParams& Params)
{
	ActualOscillationDuration = Params.bOverrideDuration ? Params.DurationOverride : OscillationDuration;

	const float EffectiveOscillationDuration = (ActualOscillationDuration > 0.f) ? ActualOscillationDuration : TNumericLimits<float>::Max();

	// init oscillations
	if (ActualOscillationDuration != 0.f)
	{
		if (OscillatorTimeRemaining > 0.f)
		{
			// this shake was already playing
			OscillatorTimeRemaining = EffectiveOscillationDuration;

			if (bBlendingOut)
			{
				bBlendingOut = false;
				CurrentBlendOutTime = 0.f;

				// stop any blendout and reverse it to a blendin
				if (OscillationBlendInTime > 0.f)
				{
					bBlendingIn = true;
					CurrentBlendInTime = OscillationBlendInTime * (1.f - CurrentBlendOutTime / OscillationBlendOutTime);
				}
				else
				{
					bBlendingIn = false;
					CurrentBlendInTime = 0.f;
				}
			}
		}
		else
		{
			RotSinOffset.X = FFOscillator::GetInitialOffset(RotOscillation.Pitch);
			RotSinOffset.Y = FFOscillator::GetInitialOffset(RotOscillation.Yaw);
			RotSinOffset.Z = FFOscillator::GetInitialOffset(RotOscillation.Roll);

			LocSinOffset.X = FFOscillator::GetInitialOffset(LocOscillation.X);
			LocSinOffset.Y = FFOscillator::GetInitialOffset(LocOscillation.Y);
			LocSinOffset.Z = FFOscillator::GetInitialOffset(LocOscillation.Z);

			FOVSinOffset = FFOscillator::GetInitialOffset(FOVOscillation);

			InitialLocSinOffset = LocSinOffset;
			InitialRotSinOffset = RotSinOffset;
			InitialFOVSinOffset = FOVSinOffset;

			OscillatorTimeRemaining = EffectiveOscillationDuration;

			if (OscillationBlendInTime > 0.f)
			{
				bBlendingIn = true;
				CurrentBlendInTime = 0.f;
			}
		}
	}

	// init cameraanim shakes
	if (AnimSequence != nullptr)
	{
		if (SequenceShakePattern == nullptr)
		{
			SequenceShakePattern = NewObject<USequenceCameraShakePattern>(this);
		}

		// Copy our anim parameters over to the sequence shake pattern.
		SequenceShakePattern->Sequence = AnimSequence;
		SequenceShakePattern->PlayRate = AnimPlayRate;
		SequenceShakePattern->Scale = AnimScale;
		SequenceShakePattern->BlendInTime = AnimBlendInTime;
		SequenceShakePattern->BlendOutTime = AnimBlendOutTime;
		SequenceShakePattern->RandomSegmentDuration = RandomAnimSegmentDuration;
		SequenceShakePattern->bRandomSegment = bRandomAnimSegment;

		// Start the sequence shake pattern.
		SequenceShakePattern->StartShakePattern(Params);
	}

	UE_LOGF(LogLegacyCameraShake, Verbose, "ULegacyCameraShake::DoStartShake %ls Duration: %f", *GetNameSafe(this), ActualOscillationDuration);

	ReceivePlayShake(ShakeScale);
}

void ULegacyCameraShake::DoUpdateShake(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	const float DeltaTime = Params.DeltaTime;

	// update oscillation times... only decrease the time remaining if we're not infinite
	if (OscillatorTimeRemaining > 0.f)
	{
		OscillatorTimeRemaining -= DeltaTime;
		OscillatorTimeRemaining = FMath::Max(0.f, OscillatorTimeRemaining);
	}
	if (bBlendingIn)
	{
		CurrentBlendInTime += DeltaTime;
	}
	if (bBlendingOut)
	{
		CurrentBlendOutTime += DeltaTime;
	}

	// see if we've crossed any important time thresholds and deal appropriately
	bool bOscillationFinished = false;

	if (OscillatorTimeRemaining <= 0.f)
	{
		// finished!
		bOscillationFinished = true;
	}
	else if (OscillatorTimeRemaining < OscillationBlendOutTime)
	{
		// start blending out
		bBlendingOut = true;
		CurrentBlendOutTime = OscillationBlendOutTime - OscillatorTimeRemaining;
	}
	else if (ActualOscillationDuration < 0.f)
	{
		// infinite oscillation, keep the time remaining up
		OscillatorTimeRemaining = TNumericLimits<float>::Max();
	}

	if (bBlendingIn)
	{
		if (CurrentBlendInTime > OscillationBlendInTime)
		{
			// done blending in!
			bBlendingIn = false;
		}
	}
	if (bBlendingOut)
	{
		if (CurrentBlendOutTime > OscillationBlendOutTime)
		{
			// done!!
			CurrentBlendOutTime = OscillationBlendOutTime;
			bOscillationFinished = true;
		}
	}

	// Do not update oscillation further if finished
	if (bOscillationFinished == false)
	{
		// calculate blend weight. calculating separately and taking the minimum handles overlapping blends nicely.
		float const BlendInWeight = (bBlendingIn) ? (CurrentBlendInTime / OscillationBlendInTime) : 1.f;
		float const BlendOutWeight = (bBlendingOut) ? (1.f - CurrentBlendOutTime / OscillationBlendOutTime) : 1.f;
		float const CurrentBlendWeight = FMath::Min(BlendInWeight, BlendOutWeight);

		// this is the oscillation scale, which includes oscillation fading
		// we'll apply the general shake scale, along with the current frame's dynamic scale, a bit later.
		float const OscillationScale = CurrentBlendWeight;

		if (OscillationScale > 0.f)
		{
			// View location offset, compute sin wave value for each component
			FVector	LocOffset = FVector(0);
			LocOffset.X = FFOscillator::UpdateOffset(LocOscillation.X, LocSinOffset.X, DeltaTime);
			LocOffset.Y = FFOscillator::UpdateOffset(LocOscillation.Y, LocSinOffset.Y, DeltaTime);
			LocOffset.Z = FFOscillator::UpdateOffset(LocOscillation.Z, LocSinOffset.Z, DeltaTime);
			LocOffset *= OscillationScale;

			OutResult.Location = LocOffset;

			// View rotation offset, compute sin wave value for each component
			FRotator RotOffset;
			RotOffset.Pitch = FFOscillator::UpdateOffset(RotOscillation.Pitch, RotSinOffset.X, DeltaTime) * OscillationScale;
			RotOffset.Yaw = FFOscillator::UpdateOffset(RotOscillation.Yaw, RotSinOffset.Y, DeltaTime) * OscillationScale;
			RotOffset.Roll = FFOscillator::UpdateOffset(RotOscillation.Roll, RotSinOffset.Z, DeltaTime) * OscillationScale;

			// Don't allow shake to flip pitch past vertical, if not using a headset (where we can't limit the camera locked to your head).
			APlayerCameraManager* CameraOwner = GetCameraManager();
			UWorld * World = (CameraOwner ? CameraOwner->GetWorld() : nullptr);
			if (!GEngine->XRSystem.IsValid() ||
				!(World != nullptr ? GEngine->XRSystem->IsHeadTrackingAllowedForWorld(*World) : GEngine->XRSystem->IsHeadTrackingAllowed()))
			{
				// Find normalized result when combined, and remove any offset that would push it past the limit.
				const float NormalizedInputPitch = FRotator::NormalizeAxis(Params.POV.Rotation.Pitch);
				RotOffset.Pitch = FRotator::NormalizeAxis(RotOffset.Pitch);
				RotOffset.Pitch = FMath::ClampAngle(NormalizedInputPitch + RotOffset.Pitch, -89.9f, 89.9f) - NormalizedInputPitch;
			}

			OutResult.Rotation = RotOffset;

			// Compute FOV change
			OutResult.FOV = OscillationScale * FFOscillator::UpdateOffset(FOVOscillation, FOVSinOffset, DeltaTime);
		}
	}

	// Update the sequence animation if there's one.
	if (SequenceShakePattern != nullptr && !SequenceShakePattern->IsFinished())
	{
		FCameraShakePatternUpdateResult ChildResult;
		SequenceShakePattern->UpdateShakePattern(Params, ChildResult);

		// The sequence shake pattern returns a local, additive result. So we should be able to
		// just combine the two results directly.
		OutResult.Location += ChildResult.Location;
		OutResult.Rotation += ChildResult.Rotation;
		OutResult.FOV += ChildResult.FOV;
		// We don't have anything else animating post-process settings so we can stomp them.
		OutResult.PostProcessSettings = ChildResult.PostProcessSettings;
		OutResult.PostProcessBlendWeight = ChildResult.PostProcessBlendWeight;
	}

	// Apply the scaling, limits, and playspace so we have an absolute result we can pass to the legacy blueprint API.
	check(OutResult.Flags == ECameraShakePatternUpdateResultFlags::Default);
	ApplyScale(Params.GetTotalScale(), OutResult);
	ApplyLimits(Params.POV, OutResult);
	ApplyPlaySpace(Params, OutResult);
	check(EnumHasAnyFlags(OutResult.Flags, ECameraShakePatternUpdateResultFlags::ApplyAsAbsolute));

	// Call the legacy blueprint API. We need to convert back and forth.
	{
		FMinimalViewInfo InOutPOV(Params.POV);
		InOutPOV.Location = OutResult.Location;
		InOutPOV.Rotation = OutResult.Rotation;
		InOutPOV.FOV = OutResult.FOV;

		BlueprintUpdateCameraShake(DeltaTime, Params.DynamicScale, InOutPOV, InOutPOV);

		OutResult.Location = InOutPOV.Location;
		OutResult.Rotation = InOutPOV.Rotation;
		OutResult.FOV = InOutPOV.FOV;
	}

	UE_LOGF(LogLegacyCameraShake, Verbose, "ULegacyCameraShake::DoUpdateShake %ls Finished: %i Duration: %f Remaining: %f", *GetNameSafe(this), bOscillationFinished, ActualOscillationDuration, OscillatorTimeRemaining);
}

void ULegacyCameraShake::DoScrubShake(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	const float NewTime = Params.AbsoluteTime;

	// reset to start and advance to desired point
	LocSinOffset = InitialLocSinOffset;
	RotSinOffset = InitialRotSinOffset;
	FOVSinOffset = InitialFOVSinOffset;

	const float EffectiveOscillationDuration = (ActualOscillationDuration > 0.f) ? ActualOscillationDuration : TNumericLimits<float>::Max();

	OscillatorTimeRemaining = EffectiveOscillationDuration;

	if (OscillationBlendInTime > 0.f)
	{
		bBlendingIn = true;
		CurrentBlendInTime = 0.f;
	}

	if (OscillationBlendOutTime > 0.f)
	{
		bBlendingOut = false;
		CurrentBlendOutTime = 0.f;
	}

	if (ActualOscillationDuration > 0.f)
	{
		if ((OscillationBlendOutTime > 0.f) && (NewTime > (ActualOscillationDuration - OscillationBlendOutTime)))
		{
			bBlendingOut = true;
			CurrentBlendOutTime = OscillationBlendOutTime - (ActualOscillationDuration - NewTime);
		}
	}

	FCameraShakePatternUpdateParams UpdateParams = Params.ToUpdateParams();

	DoUpdateShake(UpdateParams, OutResult);

	check(EnumHasAnyFlags(OutResult.Flags, ECameraShakePatternUpdateResultFlags::ApplyAsAbsolute));
}

bool ULegacyCameraShake::DoGetIsFinished() const
{
	return ((OscillatorTimeRemaining <= 0.f) &&									// oscillator is finished
		((SequenceShakePattern == nullptr) || SequenceShakePattern->IsFinished()) && // other anim is finished
		ReceiveIsFinished()														// BP thinks it's finished
		);
}

void ULegacyCameraShake::DoTeardownShake()
{
	if (SequenceShakePattern)
	{
		SequenceShakePattern->TeardownShakePattern();
	}
}

/// @cond DOXYGEN_WARNINGS

bool ULegacyCameraShake::ReceiveIsFinished_Implementation() const
{
	return true;
}

/// @endcond

bool ULegacyCameraShake::IsLooping() const
{
	return ActualOscillationDuration < 0.0f;
}

ULegacyCameraShake* ULegacyCameraShake::StartLegacyCameraShake(APlayerCameraManager* PlayerCameraManager, TSubclassOf<ULegacyCameraShake> ShakeClass, float Scale, ECameraShakePlaySpace PlaySpace, FRotator UserPlaySpaceRot)
{
	if (PlayerCameraManager)
	{
		return Cast<ULegacyCameraShake>(PlayerCameraManager->StartCameraShake(ShakeClass, Scale, PlaySpace, UserPlaySpaceRot));
	}

	return nullptr;
}

ULegacyCameraShake* ULegacyCameraShake::StartLegacyCameraShakeFromSource(APlayerCameraManager* PlayerCameraManager, TSubclassOf<ULegacyCameraShake> ShakeClass, UCameraShakeSourceComponent* SourceComponent, float Scale, ECameraShakePlaySpace PlaySpace, FRotator UserPlaySpaceRot)
{
	if (PlayerCameraManager)
	{
		return Cast<ULegacyCameraShake>(PlayerCameraManager->StartCameraShakeFromSource(ShakeClass, SourceComponent, Scale, PlaySpace, UserPlaySpaceRot));
	}

	return nullptr;
}

void ULegacyCameraShakePattern::GetShakePatternInfoImpl(FCameraShakeInfo& OutInfo) const
{
	// We will manage our own duration, but let's give a hint about how long we are for editor purposes.
	ULegacyCameraShake* Shake = GetShakeInstance<ULegacyCameraShake>();
	
	if (Shake->AnimSequence)
	{
		float PlaybackSeconds = 0.f;
		if (UMovieScene* MovieScene = Shake->AnimSequence->GetMovieScene())
		{
			const FFrameNumber PlaybackFrames = MovieScene->GetPlaybackRange().Size<FFrameNumber>();
			PlaybackSeconds = MovieScene->GetTickResolution().AsSeconds(PlaybackFrames);
		}
		const float Duration = FMath::Max(Shake->OscillationDuration, PlaybackSeconds);
		OutInfo.Duration = FCameraShakeDuration::Custom(Duration);
	}
	else
	{
		OutInfo.Duration = FCameraShakeDuration::Custom(Shake->OscillationDuration);
	}
}

void ULegacyCameraShakePattern::StopShakePatternImpl(const FCameraShakePatternStopParams& Params)
{
	ULegacyCameraShake* Shake = GetShakeInstance<ULegacyCameraShake>();
	Shake->DoStopShake(Params.bImmediately);
}

void ULegacyCameraShakePattern::StartShakePatternImpl(const FCameraShakePatternStartParams& Params)
{
	ULegacyCameraShake* Shake = GetShakeInstance<ULegacyCameraShake>();
	Shake->DoStartShake(Params);
}

void ULegacyCameraShakePattern::UpdateShakePatternImpl(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	ULegacyCameraShake* Shake = GetShakeInstance<ULegacyCameraShake>();
	Shake->DoUpdateShake(Params, OutResult);
}

void ULegacyCameraShakePattern::ScrubShakePatternImpl(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	ULegacyCameraShake* Shake = GetShakeInstance<ULegacyCameraShake>();
	Shake->DoScrubShake(Params, OutResult);
}

bool ULegacyCameraShakePattern::IsFinishedImpl() const
{
	ULegacyCameraShake* Shake = GetShakeInstance<ULegacyCameraShake>();
	return Shake->DoGetIsFinished();
}

void ULegacyCameraShakePattern::TeardownShakePatternImpl()
{
	ULegacyCameraShake* Shake = GetShakeInstance<ULegacyCameraShake>();
	Shake->DoTeardownShake();
}

