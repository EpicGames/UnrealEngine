// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shakes/PerlinNoiseCameraShakePattern.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PerlinNoiseCameraShakePattern)

float FPerlinNoiseShaker::Update(float DeltaTime, float AmplitudeMultiplier, float FrequencyMultiplier, float& InOutCurrentOffset) const
{
	const float TotalAmplitude = Amplitude * AmplitudeMultiplier;
	if (TotalAmplitude != 0.f)
	{
		InOutCurrentOffset += DeltaTime * Frequency * FrequencyMultiplier;
		return TotalAmplitude * FMath::PerlinNoise1D(InOutCurrentOffset);
	}
	return 0.f;
}

UPerlinNoiseCameraShakePattern::UPerlinNoiseCameraShakePattern(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	// Default to only location shaking.
	RotationAmplitudeMultiplier = 0.f;
	FOV.Amplitude = 0.f;
}

void UPerlinNoiseCameraShakePattern::StartShakePatternImpl(const FCameraShakePatternStartParams& Params)
{
	Super::StartShakePatternImpl(Params);

	if (!Params.bIsRestarting)
	{
		// All offsets are random. This is because the core perlin noise implementation
		// uses permutation tables, so if two shakers have the same initial offset and the same
		// frequency, they will have the same exact values.
		InitialLocationOffset = FVector3f((float)FMath::RandHelper(255), (float)FMath::RandHelper(255), (float)FMath::RandHelper(255));
		InitialRotationOffset = FVector3f((float)FMath::RandHelper(255), (float)FMath::RandHelper(255), (float)FMath::RandHelper(255));
		InitialFOVOffset = (float)FMath::RandHelper(255);

		CurrentLocationOffset = InitialLocationOffset;
		CurrentRotationOffset = InitialRotationOffset;
		CurrentFOVOffset = InitialFOVOffset;
	}
}

void UPerlinNoiseCameraShakePattern::UpdateShakePatternImpl(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	UpdatePerlinNoise(Params.DeltaTime, OutResult);

	const float BlendWeight = State.Update(Params.DeltaTime);
	OutResult.ApplyScale(BlendWeight);
}

void UPerlinNoiseCameraShakePattern::ScrubShakePatternImpl(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	// Scrubbing is like going back to our initial state and updating directly to the scrub time.
	CurrentLocationOffset = InitialLocationOffset;
	CurrentRotationOffset = InitialRotationOffset;
	CurrentFOVOffset = InitialFOVOffset;

	UpdatePerlinNoise(Params.AbsoluteTime, OutResult);

	const float BlendWeight = State.Scrub(Params.AbsoluteTime);
	OutResult.ApplyScale(BlendWeight);
}

void UPerlinNoiseCameraShakePattern::UpdatePerlinNoise(float DeltaTime, FCameraShakePatternUpdateResult& OutResult)
{
	OutResult.Location.X = X.Update(DeltaTime, LocationAmplitudeMultiplier, LocationFrequencyMultiplier, CurrentLocationOffset.X);
	OutResult.Location.Y = Y.Update(DeltaTime, LocationAmplitudeMultiplier, LocationFrequencyMultiplier, CurrentLocationOffset.Y);
	OutResult.Location.Z = Z.Update(DeltaTime, LocationAmplitudeMultiplier, LocationFrequencyMultiplier, CurrentLocationOffset.Z);

	OutResult.Rotation.Pitch = Pitch.Update(DeltaTime, RotationAmplitudeMultiplier, RotationFrequencyMultiplier, CurrentRotationOffset.X);
	OutResult.Rotation.Yaw   = Yaw.Update(  DeltaTime, RotationAmplitudeMultiplier, RotationFrequencyMultiplier, CurrentRotationOffset.Y);
	OutResult.Rotation.Roll  = Roll.Update( DeltaTime, RotationAmplitudeMultiplier, RotationFrequencyMultiplier, CurrentRotationOffset.Z);

	OutResult.FOV = FOV.Update(DeltaTime, 1.f, 1.f, CurrentFOVOffset);
}


