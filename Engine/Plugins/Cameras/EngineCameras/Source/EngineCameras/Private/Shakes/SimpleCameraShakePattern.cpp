// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shakes/SimpleCameraShakePattern.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleCameraShakePattern)

void USimpleCameraShakePattern::GetShakePatternInfoImpl(FCameraShakeInfo& OutInfo) const
{
	if (Duration > 0.f)
	{
		OutInfo.Duration = Duration;
	}
	else
	{
		OutInfo.Duration = FCameraShakeDuration::Infinite();
	}

	OutInfo.BlendIn = BlendInTime;
	OutInfo.BlendOut = BlendOutTime;
}

void USimpleCameraShakePattern::StartShakePatternImpl(const FCameraShakePatternStartParams& Params)
{
	State.Start(this, Params);
}

bool USimpleCameraShakePattern::IsFinishedImpl() const
{
	return !State.IsPlaying();
}

void USimpleCameraShakePattern::StopShakePatternImpl(const FCameraShakePatternStopParams& Params)
{
	State.Stop(Params.bImmediately);
}

void USimpleCameraShakePattern::TeardownShakePatternImpl()
{
	State = FCameraShakeState();
}

