// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shakes/WaveOscillatorCameraShakePattern.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveOscillatorCameraShakePattern)

float FWaveOscillator::Initialize(float& OutInitialOffset) const
{
	OutInitialOffset = (InitialOffsetType == EInitialWaveOscillatorOffsetType::Random)
		? FMath::FRand() * (2.f * PI)
		: 0.f;
	return Amplitude * FMath::Sin(OutInitialOffset);
}

float FWaveOscillator::Update(float DeltaTime, float AmplitudeMultiplier, float FrequencyMultiplier, float& InOutCurrentOffset) const
{
	const float TotalAmplitude = Amplitude * AmplitudeMultiplier;
	if (TotalAmplitude != 0.f)
	{
		InOutCurrentOffset += DeltaTime * Frequency * FrequencyMultiplier * (2.f * PI);
		return TotalAmplitude * FMath::Sin(InOutCurrentOffset);
	}
	return 0.f;
}

UWaveOscillatorCameraShakePattern::UWaveOscillatorCameraShakePattern(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	// Default to only location shaking.
	RotationAmplitudeMultiplier = 0.f;
	FOV.Amplitude = 0.f;
}

void UWaveOscillatorCameraShakePattern::StartShakePatternImpl(const FCameraShakePatternStartParams& Params)
{
	Super::StartShakePatternImpl(Params);

	if (!Params.bIsRestarting)
	{
		X.Initialize(InitialLocationOffset.X);
		Y.Initialize(InitialLocationOffset.Y);
		Z.Initialize(InitialLocationOffset.Z);

		CurrentLocationOffset = InitialLocationOffset;

		Pitch.Initialize(InitialRotationOffset.X);
		Yaw.Initialize(  InitialRotationOffset.Y);
		Roll.Initialize( InitialRotationOffset.Z);

		CurrentRotationOffset = InitialRotationOffset;

		FOV.Initialize(InitialFOVOffset);

		CurrentFOVOffset = InitialFOVOffset;
	}
}

void UWaveOscillatorCameraShakePattern::UpdateShakePatternImpl(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	UpdateOscillators(Params.DeltaTime, OutResult);

	const float BlendWeight = State.Update(Params.DeltaTime);
	OutResult.ApplyScale(BlendWeight);
}

void UWaveOscillatorCameraShakePattern::ScrubShakePatternImpl(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	// Scrubbing is like going back to our initial state and updating directly to the scrub time.
	CurrentLocationOffset = InitialLocationOffset;
	CurrentRotationOffset = InitialRotationOffset;
	CurrentFOVOffset = InitialFOVOffset;
	
	UpdateOscillators(Params.AbsoluteTime, OutResult);

	const float BlendWeight = State.Scrub(Params.AbsoluteTime);
	OutResult.ApplyScale(BlendWeight);
}

void UWaveOscillatorCameraShakePattern::UpdateOscillators(float DeltaTime, FCameraShakePatternUpdateResult& OutResult)
{
	OutResult.Location.X = X.Update(DeltaTime, LocationAmplitudeMultiplier, LocationFrequencyMultiplier, CurrentLocationOffset.X);
	OutResult.Location.Y = Y.Update(DeltaTime, LocationAmplitudeMultiplier, LocationFrequencyMultiplier, CurrentLocationOffset.Y);
	OutResult.Location.Z = Z.Update(DeltaTime, LocationAmplitudeMultiplier, LocationFrequencyMultiplier, CurrentLocationOffset.Z);

	OutResult.Rotation.Pitch = Pitch.Update(DeltaTime, RotationAmplitudeMultiplier, RotationFrequencyMultiplier, CurrentRotationOffset.X);
	OutResult.Rotation.Yaw   = Yaw.Update(  DeltaTime, RotationAmplitudeMultiplier, RotationFrequencyMultiplier, CurrentRotationOffset.Y);
	OutResult.Rotation.Roll  = Roll.Update( DeltaTime, RotationAmplitudeMultiplier, RotationFrequencyMultiplier, CurrentRotationOffset.Z);

	OutResult.FOV = FOV.Update(DeltaTime, 1.f, 1.f, CurrentFOVOffset);
}


