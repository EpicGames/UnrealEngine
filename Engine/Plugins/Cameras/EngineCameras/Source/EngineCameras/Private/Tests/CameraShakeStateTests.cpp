// Copyright Epic Games, Inc. All Rights Reserved.

#include "Camera/CameraShakeBase.h"
#include "Misc/AutomationTest.h"

#define LOCTEXT_NAMESPACE "CameraShakeStateTests"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraShakeStateUpdate, 
		"System.Engine.Cameras.ShakeStateUpdate", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraShakeStateUpdate::RunTest(const FString& Parameters)
{
	FCameraShakeInfo Info;
	Info.Duration = 1.f;
	Info.BlendIn = 0.2f;
	Info.BlendOut = 0.2f;

	FCameraShakeState State;
	State.Start(Info);
	UTEST_EQUAL("Update 1", State.Update(0.1f), 0.5f);
	UTEST_EQUAL("Update 2", State.Update(0.1f), 1.f);
	UTEST_EQUAL("Update 3", State.Update(0.6f), 1.f);
	UTEST_EQUAL("Update 4", State.Update(0.1f), 0.5f);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraShakeStateRestart, 
		"System.Engine.Cameras.ShakeStateRestart", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraShakeStateRestart::RunTest(const FString& Parameters)
{
	// Restarting a shake with no blending.
	{
		FCameraShakeInfo Info;
		Info.Duration = 1.f;

		FCameraShakeState State;
		State.Start(Info);
		State.Update(0.5f);
		State.Start(Info);
		UTEST_EQUAL("ElapsedTime", State.GetElapsedTime(), 0.f);
	}

	// Restarting a shake with blending.
	{
		FCameraShakeInfo Info;
		Info.Duration = 1.f;
		Info.BlendIn = 0.2f;
		Info.BlendOut = 0.3f;

		FCameraShakeState State;
		State.Start(Info);
		UTEST_EQUAL("Update", State.Update(0.85f), 0.5f);
		// We were half-way into the blend-out, so we should be half-way
		// into the blend-in as we restart. And the duration would be extended
		// by that lead-in time so that the shake still lasts 1 second overall.
		State.Start(Info);
		UTEST_TRUE("BlendingIn", State.IsBlendingIn());
		UTEST_EQUAL("BlendingInTime", State.GetCurrentBlendInTime(), 0.1f);
		UTEST_EQUAL("ElapsedTime", State.GetElapsedTime(), 0.f);
		UTEST_EQUAL("Duration", State.GetDuration(), 1.f);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraShakeStateScrub, 
		"System.Engine.Cameras.ShakeStateScrub", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraShakeStateScrub::RunTest(const FString& Parameters)
{
	FCameraShakeInfo Info;
	Info.Duration = 1.f;
	Info.BlendIn = 0.2f;
	Info.BlendOut = 0.2f;

	FCameraShakeState State;
	State.Start(Info);

	UTEST_EQUAL("Scrub 1", State.Scrub(0.1f), 0.5f);
	UTEST_TRUE("Scrub 1", State.IsPlaying());

	UTEST_EQUAL("Scrub 2", State.Scrub(0.4f), 1.f);
	UTEST_TRUE("Scrub 2", State.IsPlaying());

	UTEST_EQUAL("Scrub 3", State.Scrub(-1.f), 0.f);
	UTEST_FALSE("Scrub 3", State.IsPlaying());

	UTEST_EQUAL("Scrub 4", State.Scrub(0.2f), 1.0f);
	UTEST_TRUE("Scrub 4", State.IsPlaying());

	UTEST_EQUAL("Scrub 5", State.Scrub(2.0f), 0.0f);
	UTEST_FALSE("Scrub 5", State.IsPlaying());

	UTEST_EQUAL("Scrub 6", State.Scrub(0.9f), 0.5f);
	UTEST_TRUE("Scrub 6", State.IsPlaying());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraShakeStateDurationOverride, 
		"System.Engine.Cameras.ShakeStateDurationOverride", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraShakeStateDurationOverride::RunTest(const FString& Parameters)
{
	FCameraShakeInfo Info;
	Info.Duration = 1.f;
	Info.BlendIn = 0.2f;
	Info.BlendOut = 0.2f;

	FCameraShakeState State;
	State.Start(Info, 0.5f);

	UTEST_EQUAL("Update 1", State.Update(0.1f), 0.5f);
	UTEST_TRUE("Update 1", State.IsPlaying());

	UTEST_EQUAL("Update 2", State.Update(0.1f), 1.f);
	UTEST_TRUE("Update 2", State.IsPlaying());

	UTEST_EQUAL("Update 3", State.Update(0.1f), 1.f);
	UTEST_TRUE("Update 3", State.IsPlaying());

	UTEST_EQUAL("Update 4", State.Update(0.1f), 0.5f);
	UTEST_TRUE("Update 4", State.IsPlaying());

	UTEST_EQUAL("Update 5", State.Update(0.1f), 0.0f);
	UTEST_FALSE("Update 5", State.IsPlaying());

	return true;
}

#undef LOCTEXT_NAMESPACE
