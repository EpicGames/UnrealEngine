// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraShakeTestObjects.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraShakeTestObjects)

UConstantCameraShakePattern::UConstantCameraShakePattern(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UConstantCameraShakePattern::UpdateShakePatternImpl(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	OutResult.Location = LocationOffset;
	OutResult.Rotation = RotationOffset;

	const float BlendWeight = State.Update(Params.DeltaTime);
	OutResult.ApplyScale(BlendWeight);
}


