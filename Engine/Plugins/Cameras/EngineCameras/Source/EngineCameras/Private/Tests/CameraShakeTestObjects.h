// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Shakes/SimpleCameraShakePattern.h"
#include "CameraShakeTestObjects.generated.h"

template<typename T>
struct FTestCameraShakeAndPattern
{
	UCameraShakeBase* Shake = nullptr;
	T* Pattern = nullptr;
};

UCLASS(NotBlueprintable, HideDropdown)
class UTestCameraShake : public UCameraShakeBase
{
public:
	GENERATED_BODY()

	template<typename T>
	static FTestCameraShakeAndPattern<T> CreateWithPattern()
	{
		UTestCameraShake* Shake = NewObject<UTestCameraShake>();
		return FTestCameraShakeAndPattern<T> { Shake, Shake->ChangeRootShakePattern<T>() };
	}
};

UCLASS(NotBlueprintable, HideDropdown)
class UConstantCameraShakePattern : public USimpleCameraShakePattern
{
public:

	GENERATED_BODY()

	UConstantCameraShakePattern(const FObjectInitializer& ObjectInitializer);

	UPROPERTY()
	FVector LocationOffset;

	UPROPERTY()
	FRotator RotationOffset;

private:
	virtual void UpdateShakePatternImpl(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult) override;
};

