// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraShakeTestObjects.h"
#include "Shakes/DefaultCameraShakeBase.h"
#include "Shakes/CompositeCameraShakePattern.h"
#include "Shakes/WaveOscillatorCameraShakePattern.h"
#include "Misc/AutomationTest.h"

#define LOCTEXT_NAMESPACE "CameraShakeTests"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraShakeNullTest, 
		"System.Engine.Cameras.NullCameraShake", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraShakeNullTest::RunTest(const FString& Parameters)
{
	FMinimalViewInfo ViewInfo;
	auto TestShake = UTestCameraShake::CreateWithPattern<UConstantCameraShakePattern>();
	TestShake.Pattern->Duration = 2.f;
	TestShake.Shake->StartShake(nullptr, 1.f, ECameraShakePlaySpace::CameraLocal);
	TestShake.Shake->UpdateAndApplyCameraShake(1.f, 1.f, ViewInfo);
	UTEST_EQUAL("Location offset", ViewInfo.Location, FVector::ZeroVector);
	UTEST_EQUAL("Rotation offset", ViewInfo.Rotation, FRotator::ZeroRotator);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraShakeLocalOffsetTest,
	"System.Engine.Cameras.LocalOffsetCameraShake",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraShakeLocalOffsetTest::RunTest(const FString& Parameters)
{
	FMinimalViewInfo ViewInfo;
	ViewInfo.Location = FVector(100, 200, 50);
	ViewInfo.Rotation = FRotator(0, 90, 0);
	auto TestShake = UTestCameraShake::CreateWithPattern<UConstantCameraShakePattern>();
	TestShake.Pattern->Duration = 2.f;
	TestShake.Pattern->LocationOffset = { 10, 0, 0 };
	TestShake.Shake->StartShake(nullptr, 1.f, ECameraShakePlaySpace::CameraLocal);
	TestShake.Shake->UpdateAndApplyCameraShake(1.f, 1.f, ViewInfo);
	UTEST_EQUAL("Location offset", ViewInfo.Location, FVector(100, 210, 50));
	UTEST_EQUAL("Rotation offset", ViewInfo.Rotation, FRotator(0, 90, 0));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraShakeWorldOffsetTest,
	"System.Engine.Cameras.WorldOffsetCameraShake",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FCameraShakeWorldOffsetTest::RunTest(const FString& Parameters)
{
	FMinimalViewInfo ViewInfo;
	ViewInfo.Location = FVector(100, 200, 50);
	ViewInfo.Rotation = FRotator(0, 90, 0);
	auto TestShake = UTestCameraShake::CreateWithPattern<UConstantCameraShakePattern>();
	TestShake.Pattern->Duration = 2.f;
	TestShake.Pattern->LocationOffset = { 10, 0, 0 };
	TestShake.Shake->StartShake(nullptr, 1.f, ECameraShakePlaySpace::World);
	TestShake.Shake->UpdateAndApplyCameraShake(1.f, 1.f, ViewInfo);
	UTEST_EQUAL("Location offset", ViewInfo.Location, FVector(110, 200, 50));
	UTEST_EQUAL("Rotation offset", ViewInfo.Rotation, FRotator(0, 90, 0));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraShakeUserDefinedOffsetTest,
	"System.Engine.Cameras.UserDefinedOffsetCameraShake",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraShakeUserDefinedOffsetTest::RunTest(const FString& Parameters)
{
	FMinimalViewInfo ViewInfo;
	ViewInfo.Location = FVector(100, 200, 50);
	ViewInfo.Rotation = FRotator(0, 90, 0);
	auto TestShake = UTestCameraShake::CreateWithPattern<UConstantCameraShakePattern>();
	TestShake.Pattern->Duration = 2.f;
	TestShake.Pattern->LocationOffset = { 10, 0, 0 };
	FRotator UserPlaySpaceRot(90, 0, 0);
	TestShake.Shake->StartShake(nullptr, 1.f, ECameraShakePlaySpace::UserDefined, UserPlaySpaceRot);
	TestShake.Shake->UpdateAndApplyCameraShake(1.f, 1.f, ViewInfo);
	UTEST_EQUAL("Location offset", ViewInfo.Location, FVector(100, 200, 60));
	UTEST_EQUAL("Rotation offset", ViewInfo.Rotation, FRotator(0, 90, 0));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraShakeSingleInstanceRestartTest,
	"System.Engine.Cameras.SingleInstanceShakeRestart",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraShakeSingleInstanceRestartTest::RunTest(const FString& Parameters)
{
	UCameraShakeBase* TestShake = NewObject<UDefaultCameraShakeBase>();
	UWaveOscillatorCameraShakePattern* OscPattern = TestShake->ChangeRootShakePattern<UWaveOscillatorCameraShakePattern>();
	OscPattern->BlendInTime = 1.f;
	OscPattern->BlendOutTime = 2.f;
	OscPattern->Duration = 5.f;
	OscPattern->X.Amplitude = 8.f;
	OscPattern->X.Frequency = 1.f;
	OscPattern->X.InitialOffsetType = EInitialWaveOscillatorOffsetType::Zero;
	TestShake->bSingleInstance = true;

	// Frequency is one oscillation per second, so:
	//  0 at 0sec (0)
	//  1 at 0.25sec (PI/2)
	//  0 at 0.5sec (PI)
	// -1 at 0.75sec (3*PI/2)
	//  0 at 1sec (2*PI)

	FMinimalViewInfo ViewInfo;
	TestShake->StartShake(nullptr, 1.f, ECameraShakePlaySpace::CameraLocal);

	const FVector::FReal Tolerance =	KINDA_SMALL_NUMBER;

	// Go to 0.25sec.
	ViewInfo.Location = FVector::ZeroVector;
	ViewInfo.Rotation = FRotator::ZeroRotator;
	TestShake->UpdateAndApplyCameraShake(0.25f, 1.f, ViewInfo);
	UTEST_EQUAL_TOLERANCE("First update", ViewInfo.Location.X, 0.25f * 8.f * (FVector::FReal)FMath::Sin(PI / 2.f), Tolerance);

	// Go to 0.5sec.
	ViewInfo.Location = FVector::ZeroVector;
	ViewInfo.Rotation = FRotator::ZeroRotator;
	TestShake->UpdateAndApplyCameraShake(0.25f, 1.f, ViewInfo);
	UTEST_EQUAL_TOLERANCE("Second update", ViewInfo.Location.X, 0.5f * 8.f * (FVector::FReal)FMath::Sin(PI), Tolerance);

	// Go to 1sec.
	ViewInfo.Location = FVector::ZeroVector;
	ViewInfo.Rotation = FRotator::ZeroRotator;
	TestShake->UpdateAndApplyCameraShake(0.5f, 1.f, ViewInfo);
	UTEST_EQUAL_TOLERANCE("Third update", ViewInfo.Location.X, 8.f * (FVector::FReal)FMath::Sin(2.f * PI), Tolerance);

	// Go to 4sec.
	ViewInfo.Location = FVector::ZeroVector;
	ViewInfo.Rotation = FRotator::ZeroRotator;
	TestShake->UpdateAndApplyCameraShake(3.f, 1.f, ViewInfo);
	UTEST_EQUAL_TOLERANCE("Fourth update", ViewInfo.Location.X, 0.5f * 8.f * (FVector::FReal)FMath::Sin(8.f * PI), Tolerance);

	// Restart in the middle of the blend-out... we were at 50% so it should reset us
	// at the equivalent point in the blend-in.
	TestShake->StartShake(nullptr, 1.f, ECameraShakePlaySpace::CameraLocal);
	
	// Go to 0.25sec (but blend-in started at 50% this time, so it will be at 75%).
	ViewInfo.Location = FVector::ZeroVector;
	ViewInfo.Rotation = FRotator::ZeroRotator;
	TestShake->UpdateAndApplyCameraShake(0.25f, 1.f, ViewInfo);
	UTEST_EQUAL_TOLERANCE("Fifth update", ViewInfo.Location.X, 0.75f * 8.f * (FVector::FReal)FMath::Sin(PI / 2.f), Tolerance);

	// Go to 0.5sec (but now the blend-in is finished).
	ViewInfo.Location = FVector::ZeroVector;
	ViewInfo.Rotation = FRotator::ZeroRotator;
	TestShake->UpdateAndApplyCameraShake(0.25f, 1.f, ViewInfo);
	UTEST_EQUAL_TOLERANCE("Sixth update", ViewInfo.Location.X, 8.f * (FVector::FReal)FMath::Sin(PI), Tolerance);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompositeCameraShakeRunTest,
	"System.Engine.Cameras.CompositeCameraShakeRunTest",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCompositeCameraShakeRunTest::RunTest(const FString& Parameters)
{
	auto TestShake = UTestCameraShake::CreateWithPattern<UCompositeCameraShakePattern>();

	UConstantCameraShakePattern* ChildPattern1 = TestShake.Pattern->AddChildPattern<UConstantCameraShakePattern>();
	ChildPattern1->Duration = 1.f;
	ChildPattern1->BlendInTime = ChildPattern1->BlendOutTime = 0.2f;
	ChildPattern1->LocationOffset = FVector(1.f, 0, 0);

	UConstantCameraShakePattern* ChildPattern2 = TestShake.Pattern->AddChildPattern<UConstantCameraShakePattern>();
	ChildPattern2->Duration = 2.f;
	ChildPattern2->BlendInTime = ChildPattern2->BlendOutTime = 0.3f;
	ChildPattern2->LocationOffset = FVector(1.f, 0, 0);

	// First run: letting it go until the end.
	{
		FMinimalViewInfo ViewInfo;
		TestShake.Shake->StartShake(nullptr, 1.f, ECameraShakePlaySpace::CameraLocal);
		UTEST_FALSE("Composite shake is still active", TestShake.Shake->IsFinished());

		// t=0.1 : 50% into first pattern's blend-in, 33% into the second's.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.1f, 1.f, ViewInfo);
		UTEST_FALSE("Composite shake is still active", TestShake.Shake->IsFinished());
		UTEST_EQUAL("Both children patterns are blending in", ViewInfo.Location, FVector((0.5f + 1.f/3.f), 0, 0));

		// t=0.5 : both patterns are applied in full.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.4f, 1.f, ViewInfo);
		UTEST_FALSE("Composite shake is still active", TestShake.Shake->IsFinished());
		UTEST_EQUAL("Both children patterns active", ViewInfo.Location, FVector(2, 0, 0));

		// t=0.9 : first pattern is back down to 50%.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.4f, 1.f, ViewInfo);
		UTEST_FALSE("Composite shake is still active", TestShake.Shake->IsFinished());
		UTEST_EQUAL("First child pattern is blending out", ViewInfo.Location, FVector(0.5f + 1.f, 0, 0));

		// t=1 : first pattern has ended.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.1f, 1.f, ViewInfo);
		UTEST_FALSE("Composite shake is still active", TestShake.Shake->IsFinished());
		UTEST_EQUAL("First child pattern has ended", ViewInfo.Location, FVector(1, 0, 0));

		// t=1.85 : second pattern is back down to 50%.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.85f, 1.f, ViewInfo);
		UTEST_FALSE("Composite shake is still active", TestShake.Shake->IsFinished());
		UTEST_EQUAL("Second child pattern is blending out", ViewInfo.Location, FVector(0.5f, 0, 0));

		// t=2 : second pattern has ended.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.85f, 1.f, ViewInfo);
		UTEST_TRUE("Composite shake has ended", TestShake.Shake->IsFinished());
		UTEST_EQUAL("Both children patterns have ended", ViewInfo.Location, FVector(0, 0, 0));

		TestShake.Shake->TeardownShake();
	}

	// Second run: stopping while both shakes are active.
	{
		FMinimalViewInfo ViewInfo;
		TestShake.Shake->StartShake(nullptr, 1.f, ECameraShakePlaySpace::CameraLocal);

		// t=0.5 : both patterns are active in full.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.5f, 1.f, ViewInfo);

		TestShake.Shake->StopShake(false);

		// Stopping placed us at the beginning of the longest blend out. Advancing by 0.1s should
		// put us 50% into the first pattern's blend out, and 33% into the second pattern's blend out.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.1f, 1.f, ViewInfo);
		UTEST_FALSE("Composite shake is still active", TestShake.Shake->IsFinished());
		UTEST_EQUAL("Both patterns are blending out", ViewInfo.Location, FVector(0.5f + 2.f/3.f, 0, 0));

		// Advancing by another 0.1s ends the first pattern, and we're 66% into the second's blend out.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.1f, 1.f, ViewInfo);
		UTEST_FALSE("Composite shake is still active", TestShake.Shake->IsFinished());
		UTEST_EQUAL("Second pattern is blending out", ViewInfo.Location, FVector(1.f/3.f, 0, 0));

		// Advancing by the last 0.1s puts us at the end.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.1f, 1.f, ViewInfo);
		UTEST_TRUE("Composite shake has ended", TestShake.Shake->IsFinished());
		UTEST_EQUAL("Both children patterns ended", ViewInfo.Location, FVector(0, 0, 0));

		TestShake.Shake->TeardownShake();
	}

	// Third run: stopping while the first shake is blending out, and the second shake is active.
	{
		FMinimalViewInfo ViewInfo;
		TestShake.Shake->StartShake(nullptr, 1.f, ECameraShakePlaySpace::CameraLocal);

		// t=0.85 : the first shake is at 25% of its blend out.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.85f, 1.f, ViewInfo);

		TestShake.Shake->StopShake(false);

		// t=0.95 : the first pattern is at 75% of its blend out.
		// The second shake started blending out 0.1s earlier, so it's at 33% of its blend out.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.1f, 1.f, ViewInfo);
		UTEST_FALSE("Composite shake is still active", TestShake.Shake->IsFinished());
		UTEST_EQUAL("Both children pattern are blending out", ViewInfo.Location, FVector(0.25f + 2.f/3.f, 0, 0));

		// t=1 : the first pattern has ended, the second pattern is at 50%.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.05f, 1.f, ViewInfo);
		UTEST_FALSE("Composite shake is still active", TestShake.Shake->IsFinished());
		UTEST_EQUAL("Second child pattern is blending out", ViewInfo.Location, FVector(0.5f, 0, 0));

		// t=2.5 : both patterns have ended.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(1.5f, 1.f, ViewInfo);
		UTEST_TRUE("Composite shake has ended", TestShake.Shake->IsFinished());
		UTEST_EQUAL("Both children patterns ended", ViewInfo.Location, FVector(0, 0, 0));

		TestShake.Shake->TeardownShake();
	}

	// Fourth run: stopping while only the second shake is active.
	{
		FMinimalViewInfo ViewInfo;
		TestShake.Shake->StartShake(nullptr, 1.f, ECameraShakePlaySpace::CameraLocal);

		// t=1.1 : only the second pattern is active.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(1.1f, 1.f, ViewInfo);

		TestShake.Shake->StopShake(false);

		// Stopping has placed us at the beginning of the blend out. Advancing by 0.15s should
		// place us in the middle of the blend out.
		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.15f, 1.f, ViewInfo);
		UTEST_FALSE("Composite shake is still active", TestShake.Shake->IsFinished());
		UTEST_EQUAL("Second child pattern is blending out", ViewInfo.Location, FVector(0.5f, 0, 0));

		ViewInfo.Location = FVector::ZeroVector;
		TestShake.Shake->UpdateAndApplyCameraShake(0.15f, 1.f, ViewInfo);
		UTEST_TRUE("Composite shake has ended", TestShake.Shake->IsFinished());
		UTEST_EQUAL("Both children patterns ended", ViewInfo.Location, FVector(0, 0, 0));

		TestShake.Shake->TeardownShake();
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraShakeImmediatelyStopInfiniteTest,
	"System.Engine.Cameras.ImmediatelyStopInfiniteCameraShake",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraShakeImmediatelyStopInfiniteTest::RunTest(const FString& Parameters)
{
	FMinimalViewInfo ViewInfo;
	ViewInfo.Location = FVector::ZeroVector;
	ViewInfo.Rotation = FRotator::ZeroRotator;
	auto TestShake = UTestCameraShake::CreateWithPattern<UConstantCameraShakePattern>();
	TestShake.Pattern->Duration = -1.f; // Negative means infinite duration
	TestShake.Pattern->LocationOffset = { 10, 0, 0 };

	TestShake.Shake->StartShake(nullptr, 1.f, ECameraShakePlaySpace::CameraLocal);
	UTEST_FALSE("Infinite shake is running", TestShake.Shake->IsFinished());
	UTEST_TRUE("Infinite shake is active", TestShake.Shake->IsActive());

	ViewInfo.Location = FVector::ZeroVector;
	TestShake.Shake->UpdateAndApplyCameraShake(99.f, 1.f, ViewInfo);
	UTEST_FALSE("Infinite shake is still running", TestShake.Shake->IsFinished());
	UTEST_TRUE("Infinite shake is still active", TestShake.Shake->IsActive());
	UTEST_EQUAL("Location offset", ViewInfo.Location, FVector(10, 0, 0));
	
	ViewInfo.Location = FVector::ZeroVector;
	TestShake.Shake->UpdateAndApplyCameraShake(99.f, 1.f, ViewInfo);
	UTEST_FALSE("Infinite shake is still running", TestShake.Shake->IsFinished());
	UTEST_TRUE("Infinite shake is still active", TestShake.Shake->IsActive());
	UTEST_EQUAL("Location offset", ViewInfo.Location, FVector(10, 0, 0));

	TestShake.Shake->StopShake(true);
	UTEST_TRUE("Infinite shake is finished", TestShake.Shake->IsFinished());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCameraShakeBlendOutInfiniteTest,
	"System.Engine.Cameras.BlendOutInfiniteCameraShake",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FCameraShakeBlendOutInfiniteTest::RunTest(const FString& Parameters)
{
	FMinimalViewInfo ViewInfo;
	ViewInfo.Location = FVector::ZeroVector;
	ViewInfo.Rotation = FRotator::ZeroRotator;
	auto TestShake = UTestCameraShake::CreateWithPattern<UConstantCameraShakePattern>();
	TestShake.Pattern->Duration = -1.f; // Negative means infinite duration
	TestShake.Pattern->LocationOffset = { 10, 0, 0 };
	TestShake.Pattern->BlendOutTime = 1.f; // One second blend out

	TestShake.Shake->StartShake(nullptr, 1.f, ECameraShakePlaySpace::CameraLocal);
	UTEST_FALSE("Infinite shake is running", TestShake.Shake->IsFinished());
	UTEST_TRUE("Infinite shake is active", TestShake.Shake->IsActive());

	ViewInfo.Location = FVector::ZeroVector;
	TestShake.Shake->UpdateAndApplyCameraShake(99.f, 1.f, ViewInfo);
	UTEST_FALSE("Infinite shake is still running", TestShake.Shake->IsFinished());
	UTEST_TRUE("Infinite shake is still active", TestShake.Shake->IsActive());
	UTEST_EQUAL("Location offset", ViewInfo.Location, FVector(10, 0, 0));
	
	ViewInfo.Location = FVector::ZeroVector;
	TestShake.Shake->StopShake(false);
	TestShake.Shake->UpdateAndApplyCameraShake(0.5f, 1.f, ViewInfo);
	UTEST_FALSE("Infinite shake is still running", TestShake.Shake->IsFinished());
	UTEST_TRUE("Infinite shake is still active", TestShake.Shake->IsActive());
	UTEST_EQUAL("Location offset", ViewInfo.Location, FVector(5, 0, 0));

	ViewInfo.Location = FVector::ZeroVector;
	TestShake.Shake->UpdateAndApplyCameraShake(0.5f, 1.f, ViewInfo);
	UTEST_TRUE("Infinite shake is finished", TestShake.Shake->IsFinished());
	UTEST_EQUAL("Location offset", ViewInfo.Location, FVector(0, 0, 0));

	return true;
}

#undef LOCTEXT_NAMESPACE
