// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraModifier.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "CameraAnimationCameraModifier.generated.h"

#define UE_API ENGINECAMERAS_API

enum class ECameraShakePlaySpace : uint8;

class UCameraAnimationSequence;
class UCameraAnimationSequenceCameraStandIn;
class UCameraAnimationSequencePlayer;
struct FMinimalViewInfo;

UENUM()
enum class ECameraAnimationPlaySpace : uint8
{
	/** This anim is applied in camera space. */
	CameraLocal,
	/** This anim is applied in world space. */
	World,
	/** This anim is applied in a user-specified space (defined by UserPlaySpaceMatrix). */
	UserDefined,
};

UENUM()
enum class ECameraAnimationEasingType : uint8
{
	Linear,
	Sinusoidal,
	Quadratic,
	Cubic,
	Quartic,
	Quintic,
	Exponential,
	Circular,
};

/** Parameter struct for adding new camera animations to UCameraAnimationCameraModifier */
USTRUCT(BlueprintType)
struct FCameraAnimationParams
{
	GENERATED_BODY()

	/** Time scale for playing the new camera animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Animation")
	float PlayRate = 1.f;
	/** Global scale to use for the new camera animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Animation")
	float Scale = 1.f;

	/** Ease-in function type */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Animation")
	ECameraAnimationEasingType EaseInType = ECameraAnimationEasingType::Linear;
	/** Ease-in duration in seconds */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Animation")
	float EaseInDuration = 0.f;

	/** Ease-out function type */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Animation")
	ECameraAnimationEasingType EaseOutType = ECameraAnimationEasingType::Linear;
	/** Ease-out duration in seconds */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Animation")
	float EaseOutDuration = 0.f;

	/** Whether the camera animation should loop */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Animation")
	bool bLoop = false;
	/** Offset, in frames, into the animation to start at */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Animation")
	int32 StartOffset = 0;
	/** Whether the camera animation should have a random start time */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Animation")
	bool bRandomStartTime = false;
	/** Override the duration of the animation with a new duration (including blends) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Animation")
	float DurationOverride = 0.f;

	/** The transform space to use for the new camera shake */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Animation")
	ECameraAnimationPlaySpace PlaySpace = ECameraAnimationPlaySpace::CameraLocal;
	/** User space to use when PlaySpace is UserDefined */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Animation")
	FRotator UserPlaySpaceRot = FRotator::ZeroRotator;
};

/** A handle to a camera animation running in UCameraAnimationCameraModifier */
USTRUCT(BlueprintType)
struct FCameraAnimationHandle
{
	GENERATED_BODY()

	static FCameraAnimationHandle Invalid;

	FCameraAnimationHandle() : InstanceID(MAX_uint16), InstanceSerial(0) {}
	FCameraAnimationHandle(uint16 InInstanceID, uint16 InInstanceSerial) : InstanceID(InInstanceID), InstanceSerial(InInstanceSerial) {}

	bool IsValid() const
	{
		return InstanceID != MAX_uint16;
	}

	friend bool operator==(FCameraAnimationHandle A, FCameraAnimationHandle B)
	{
		return A.InstanceID == B.InstanceID && A.InstanceSerial == B.InstanceSerial;
	}
	friend bool operator!=(FCameraAnimationHandle A, FCameraAnimationHandle B)
	{
		return !(A == B);
	}
	friend bool operator<(FCameraAnimationHandle A, FCameraAnimationHandle B)
	{
		return A.InstanceID < B.InstanceID;
	}
	friend uint32 GetTypeHash(FCameraAnimationHandle In)
	{
		return HashCombine(In.InstanceID, In.InstanceSerial);
	}

private:
	uint16 InstanceID;
	uint16 InstanceSerial;

	friend class UCameraAnimationCameraModifier;
};

/**
 * Information about an active camera animation inside UCameraAnimationCameraModifier.
 */
USTRUCT()
struct FActiveCameraAnimationInfo
{
	GENERATED_BODY()

	UE_API FActiveCameraAnimationInfo();

	/** Whether this is a valid, ongoing camera animation */
	UE_API bool IsValid() const;

	/** Whether this camera animation's player is valid */
	UE_API bool HasValidPlayer() const;

	/** The sequence to use for the animation. */
	UPROPERTY()
	TObjectPtr<UCameraAnimationSequence> Sequence;

	/** The parameters for playing the animation. */
	UPROPERTY()
	FCameraAnimationParams Params;

	/** A reference handle for use with UCameraAnimationCameraModifier. */
	UPROPERTY()
	FCameraAnimationHandle Handle;

	/** The player for playing the animation. */
	UPROPERTY(Transient)
	TObjectPtr<UCameraAnimationSequencePlayer> Player;

	/** Standin for the camera actor and components */
	UPROPERTY(Transient)
	TObjectPtr<UCameraAnimationSequenceCameraStandIn> CameraStandIn;

	/** Current time into easing in */
	UPROPERTY()
	float EaseInCurrentTime;

	/** Current time into easing out */
	UPROPERTY()
	float EaseOutCurrentTime;

	/** Whether easing in is ongoing */
	UPROPERTY()
	bool bIsEasingIn;

	/** Whether easing out is ongoing */
	UPROPERTY()
	bool bIsEasingOut;
};

/**
 * A camera modifier that plays camera animation sequences.
 */
UCLASS(MinimalAPI, config=Camera)
class UCameraAnimationCameraModifier : public UCameraModifier
{
	GENERATED_BODY()

public:
	UE_API UCameraAnimationCameraModifier(const FObjectInitializer& ObjectInitializer);

	/**
	 * Play a new camera animation sequence.
	 * @param Sequence		The sequence to use for the new camera animation.
	 * @param Params		The parameters for the new camera animation instance.
	 */
	UFUNCTION(BlueprintCallable, Category="Camera Animation")
	UE_API FCameraAnimationHandle PlayCameraAnimation(UCameraAnimationSequence* Sequence, FCameraAnimationParams Params);

	/**
	 * Returns whether the given camera animation is playing.
	 * @param Handle		A handle to a previously started camera animation.
	 * @return				Whether the corresponding camera animation is playing or not.
	 */
	UFUNCTION(BlueprintCallable, Category="Camera Animation")
	UE_API bool IsCameraAnimationActive(const FCameraAnimationHandle& Handle) const;
	
	/** 
	 * Stops the given camera animation instance.
	 * @param Hanlde		A handle to a previously started camera animation.
	 * @param bImmediate	True to stop it right now and ignore blend out, false to let it blend out as indicated.
	 */
	UFUNCTION(BlueprintCallable, Category = "Camera Animation")
	UE_API void StopCameraAnimation(const FCameraAnimationHandle& Handle, bool bImmediate = false);
	
	/**
	 * Stop playing all instances of the given camera animation sequence.
	 * @param Sequence		The sequence of which to stop all instances.
	 * @param bImmediate	True to stop it right now and ignore blend out, false to let it blend out as indicated.
	 */
	UFUNCTION(BlueprintCallable, Category="Camera Animation")
	UE_API void StopAllCameraAnimationsOf(UCameraAnimationSequence* Sequence, bool bImmediate = false);
	
	/**
	 * Stop all camera animation instances.
	 * @param bImmediate	True to stop it right now and ignore blend out, false to let it blend out as indicated.
	 */
	UFUNCTION(BlueprintCallable, Category="Camera Animation")
	UE_API virtual void StopAllCameraAnimations(bool bImmediate = false);

	// UCameraModifier interface
	UE_API virtual bool ModifyCamera(float DeltaTime, FMinimalViewInfo& InOutPOV) override;
	UE_API virtual void DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos) override;

public:

	UFUNCTION(BlueprintPure, Category="Camera Animation", meta=(WorldContext="WorldContextObject"))
	static UE_API UCameraAnimationCameraModifier* GetCameraAnimationCameraModifier(const UObject* WorldContextObject, int32 PlayerIndex);

	UFUNCTION(BlueprintPure, Category="Camera Animation", meta=(WorldContext="WorldContextObject"))
	static UE_API UCameraAnimationCameraModifier* GetCameraAnimationCameraModifierFromID(const UObject* WorldContextObject, int32 ControllerID);

	UFUNCTION(BlueprintPure, Category="Camera Animation")
	static UE_API UCameraAnimationCameraModifier* GetCameraAnimationCameraModifierFromPlayerController(const APlayerController* PlayerController);

protected:

	static UE_API float EvaluateEasing(ECameraAnimationEasingType EasingType, float Interp);

	UE_API int32 FindInactiveCameraAnimation();
	UE_API const FActiveCameraAnimationInfo* GetActiveCameraAnimation(const FCameraAnimationHandle& Handle) const;
	UE_API FActiveCameraAnimationInfo* GetActiveCameraAnimation(const FCameraAnimationHandle& Handle);
	UE_API void DeactivateCameraAnimation(int32 Index);

	UE_API void TickAllAnimations(float DeltaTime, FMinimalViewInfo& InOutPOV);
	UE_API void TickAnimation(FActiveCameraAnimationInfo& CameraAnimation, float DeltaTime, FMinimalViewInfo& InOutPOV);

protected:

	/** List of active camera animation instances */
	UPROPERTY()
	TArray<FActiveCameraAnimationInfo> ActiveAnimations;

	/** Next serial number to use for a camera animation instance */
	UPROPERTY()
	uint16 NextInstanceSerialNumber;
};

/**
 * Blueprint function library for autocasting a player camera manager into the camera animation camera modifier.
 * This prevents breaking Blueprints now that APlayerCameraManager::StartCameraShake returns the base class.
 */
UCLASS(MinimalAPI)
class UEngineCameraAnimationFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintPure, Category="Camera Animation", meta=(BlueprintAutocast))
	static UE_API UCameraAnimationCameraModifier* Conv_CameraAnimationCameraModifier(APlayerCameraManager* PlayerCameraManager);

	UFUNCTION(BlueprintPure, Category = "Camera Animation", meta = (BlueprintAutocast))
	static UE_API ECameraShakePlaySpace Conv_CameraShakePlaySpace(ECameraAnimationPlaySpace CameraAnimationPlaySpace);

	UFUNCTION(BlueprintPure, Category = "Camera Animation", meta = (BlueprintAutocast))
	static UE_API ECameraAnimationPlaySpace Conv_CameraAnimationPlaySpace(ECameraShakePlaySpace CameraShakePlaySpace);
};

//UE_DEPRECATED(5.5, "Please use UEngineCameraAnimationFunctionLibrary")
class UGameplayCamerasFunctionLibrary : public UEngineCameraAnimationFunctionLibrary
{
};

#undef UE_API
