// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/Platform.h"
#include "Modules/ModuleInterface.h"
#include "Stats/Stats.h"

class IEngineCamerasModule : public IModuleInterface
{
};

DECLARE_STATS_GROUP(TEXT("Camera Animation Evaluation"), STATGROUP_CameraAnimation, STATCAT_Advanced)

