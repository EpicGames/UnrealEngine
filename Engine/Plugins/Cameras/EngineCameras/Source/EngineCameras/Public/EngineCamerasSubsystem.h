// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "EngineCamerasSubsystem.generated.h"

#define UE_API ENGINECAMERAS_API

class UCameraAnimationSequence;
struct FCameraAnimationHandle;
struct FCameraAnimationParams;

/**
 * World subsystem that holds global objects for handling camera animation sequences.
 */
UCLASS(MinimalAPI)
class UEngineCamerasSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:

	/** Get the camera animation sequence subsystem for the given world */
	static UE_API UEngineCamerasSubsystem* GetEngineCamerasSubsystem(const UWorld* InWorld);

public:

	/**
	 * Play a new camera animation sequence.
	 * @param PlayerController The player controller on which to play the animation.
	 * @param Sequence		The sequence to use for the new camera animation.
	 * @param Params		The parameters for the new camera animation instance.
	 */
	UFUNCTION(BlueprintCallable, Category="Camera Animation")
	UE_API FCameraAnimationHandle PlayCameraAnimation(APlayerController* PlayerController, UCameraAnimationSequence* Sequence, FCameraAnimationParams Params);

	/**
	 * Returns whether the given camera animation is playing.
	 * @param PlayerController The player controller on which to play the animation.
	 * @param Handle		A handle to a previously started camera animation.
	 * @return				Whether the corresponding camera animation is playing or not.
	 */
	UFUNCTION(BlueprintPure, Category="Camera Animation")
	UE_API bool IsCameraAnimationActive(APlayerController* PlayerController, const FCameraAnimationHandle& Handle) const;

	/** 
	 * Stops the given camera animation instance.
	 * @param PlayerController The player controller on which to play the animation.
	 * @param Hanlde		A handle to a previously started camera animation.
	 * @param bImmediate	True to stop it right now and ignore blend out, false to let it blend out as indicated.
	 */
	UFUNCTION(BlueprintCallable, Category = "Camera Animation")
	UE_API void StopCameraAnimation(APlayerController* PlayerController, const FCameraAnimationHandle& Handle, bool bImmediate = false);
	
	/**
	 * Stop playing all instances of the given camera animation sequence.
	 * @param PlayerController The player controller on which to play the animation.
	 * @param Sequence		The sequence of which to stop all instances.
	 * @param bImmediate	True to stop it right now and ignore blend out, false to let it blend out as indicated.
	 */
	UFUNCTION(BlueprintCallable, Category="Camera Animation")
	UE_API void StopAllCameraAnimationsOf(APlayerController* PlayerController, UCameraAnimationSequence* Sequence, bool bImmediate = false);
	
	/**
	 * Stop all camera animation instances.
	 * @param PlayerController The player controller on which to play the animation.
	 * @param bImmediate	True to stop it right now and ignore blend out, false to let it blend out as indicated.
	 */
	UFUNCTION(BlueprintCallable, Category="Camera Animation")
	UE_API void StopAllCameraAnimations(APlayerController* PlayerController, bool bImmediate = false);
};

#undef UE_API
