// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraShakeBase.h"
#include "CompositeCameraShakePattern.generated.h"

#define UE_API ENGINECAMERAS_API

/**
 * A base class for a simple camera shake.
 */
UCLASS(MinimalAPI, meta=(AutoExpandCategories="CameraShake"))
class UCompositeCameraShakePattern : public UCameraShakePattern
{
public:

	GENERATED_BODY()

	UCompositeCameraShakePattern(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

	template<typename PatternClass>
	PatternClass* AddChildPattern()
	{
		PatternClass* NewChildPattern = NewObject<PatternClass>();
		ChildPatterns.Add(NewChildPattern);
		return NewChildPattern;
	}

public:

	/** The list of child shake patterns */
	UPROPERTY(EditAnywhere, Instanced, Category=CameraShake)
	TArray<TObjectPtr<UCameraShakePattern>> ChildPatterns;

private:

	// UCameraShakePattern interface
	UE_API virtual void GetShakePatternInfoImpl(FCameraShakeInfo& OutInfo) const override;
	UE_API virtual void StartShakePatternImpl(const FCameraShakePatternStartParams& Params) override;
	UE_API virtual void UpdateShakePatternImpl(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult) override;
	UE_API virtual void ScrubShakePatternImpl(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult) override;
	UE_API virtual bool IsFinishedImpl() const override;
	UE_API virtual void StopShakePatternImpl(const FCameraShakePatternStopParams& Params) override;
	UE_API virtual void TeardownShakePatternImpl() override;
};

#undef UE_API
