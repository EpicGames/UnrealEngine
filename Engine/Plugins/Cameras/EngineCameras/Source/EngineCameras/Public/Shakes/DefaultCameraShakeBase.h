// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraShakeBase.h"
#include "DefaultCameraShakeBase.generated.h"

#define UE_API ENGINECAMERAS_API

/**
 * Like UCameraShakeBase but with a perlin noise shake pattern by default, for convenience.
 */
UCLASS(MinimalAPI)
class UDefaultCameraShakeBase : public UCameraShakeBase
{
	GENERATED_BODY()

public:

	UE_API UDefaultCameraShakeBase(const FObjectInitializer& ObjInit);
};

#undef UE_API
