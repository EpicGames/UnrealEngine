// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraShakeBase.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "LegacyCameraShake.generated.h"

#define UE_API ENGINECAMERAS_API

class AActor;
class UCameraShakeSourceComponent;

/************************************************************
 * Parameters for defining oscillating camera shakes
 ************************************************************/

 /** Types of waveforms that can be used for camera shake oscillators */
UENUM(BlueprintType)
enum class EOscillatorWaveform : uint8
{
	/** A sinusoidal wave */
	SineWave,

	/** Perlin noise */
	PerlinNoise,
};

/** Shake start offset parameter */
UENUM()
enum EInitialOscillatorOffset : int
{
	/** Start with random offset (default). */
	EOO_OffsetRandom UMETA(DisplayName = "Random"),
	/** Start with zero offset. */
	EOO_OffsetZero UMETA(DisplayName = "Zero"),
	EOO_MAX,
};

/** Defines oscillation of a single number. */
USTRUCT(BlueprintType)
struct FFOscillator
{
	GENERATED_USTRUCT_BODY()

	/** Amplitude of the sinusoidal oscillation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FOscillator)
	float Amplitude;

	/** Frequency of the sinusoidal oscillation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FOscillator)
	float Frequency;

	/** Defines how to begin (either at zero, or at a randomized value. */
	UPROPERTY(EditAnywhere, Category = FOscillator)
	TEnumAsByte<enum EInitialOscillatorOffset> InitialOffset;

	/** Type of waveform to use for oscillation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FOscillator)
	EOscillatorWaveform Waveform;

	FFOscillator()
		: Amplitude(0)
		, Frequency(0)
		, InitialOffset(0)
		, Waveform(EOscillatorWaveform::SineWave)
	{}

	/** Advances the oscillation time and returns the current value. */
	static UE_API float UpdateOffset(FFOscillator const& Osc, float& CurrentOffset, float DeltaTime);

	/** Advances the oscillation time and returns the current value. */
	static UE_API float UpdateOffset(FFOscillator const& Osc, double& CurrentOffset, float DeltaTime);

	/** Returns the initial value of the oscillator. */
	static UE_API float GetInitialOffset(FFOscillator const& Osc);

	/** Returns the offset at the given time */
	static UE_API float GetOffsetAtTime(FFOscillator const& Osc, float InitialOffset, float Time);
};

/** Defines FRotator oscillation. */
USTRUCT(BlueprintType)
struct FROscillator
{
	GENERATED_USTRUCT_BODY()

	/** Pitch oscillation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ROscillator)
	struct FFOscillator Pitch;

	/** Yaw oscillation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ROscillator)
	struct FFOscillator Yaw;

	/** Roll oscillation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ROscillator)
	struct FFOscillator Roll;

};

/** Defines FVector oscillation. */
USTRUCT(BlueprintType)
struct FVOscillator
{
	GENERATED_USTRUCT_BODY()

	/** Oscillation in the X axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = VOscillator)
	struct FFOscillator X;

	/** Oscillation in the Y axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = VOscillator)
	struct FFOscillator Y;

	/** Oscillation in the Z axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = VOscillator)
	struct FFOscillator Z;
};


/**
 * Legacy camera shake which can do either oscillation or run camera anims.
 */
UCLASS(MinimalAPI, Blueprintable, HideCategories = (CameraShakePattern))
class ULegacyCameraShake : public UCameraShakeBase
{
	GENERATED_UCLASS_BODY()

public:
	/** Duration in seconds of current screen shake. Less than 0 means indefinite, 0 means no oscillation. */
	UPROPERTY(EditAnywhere, Category = Oscillation)
	float OscillationDuration;

	/** Duration of the blend-in, where the oscillation scales from 0 to 1. */
	UPROPERTY(EditAnywhere, Category = Oscillation, meta = (ClampMin = "0.0"))
	float OscillationBlendInTime;

	/** Duration of the blend-out, where the oscillation scales from 1 to 0. */
	UPROPERTY(EditAnywhere, Category = Oscillation, meta = (ClampMin = "0.0"))
	float OscillationBlendOutTime;

	/** Rotational oscillation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Oscillation)
	struct FROscillator RotOscillation;

	/** Positional oscillation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Oscillation)
	struct FVOscillator LocOscillation;

	/** FOV oscillation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Oscillation)
	struct FFOscillator FOVOscillation;

	/************************************************************
	 * Parameters for defining sequencer-driven camera shakes
	 ************************************************************/

	 /** Scalar defining how fast to play the anim. */
	UPROPERTY(EditAnywhere, Category = AnimShake, meta = (ClampMin = "0.001"))
	float AnimPlayRate;

	/** Scalar defining how "intense" to play the anim. */
	UPROPERTY(EditAnywhere, Category = AnimShake, meta = (ClampMin = "0.0"))
	float AnimScale;

	/** Linear blend-in time. */
	UPROPERTY(EditAnywhere, Category = AnimShake, meta = (ClampMin = "0.0"))
	float AnimBlendInTime;

	/** Linear blend-out time. */
	UPROPERTY(EditAnywhere, Category = AnimShake, meta = (ClampMin = "0.0"))
	float AnimBlendOutTime;

	/** When bRandomAnimSegment is true, this defines how long the anim should play. */
	UPROPERTY(EditAnywhere, Category = AnimShake, meta = (ClampMin = "0.0", editcondition = "bRandomAnimSegment"))
	float RandomAnimSegmentDuration;

	/** Source camera animation sequence to play. Can be null. */
	UPROPERTY(EditAnywhere, Category = AnimShake)
	TObjectPtr<class UCameraAnimationSequence> AnimSequence;

	/**
	* If true, play a random snippet of the animation of length Duration.  Implies bLoop and bRandomStartTime = true for the AnimSequence.
	* If false, play the full anim once, non-looped. Useful for getting variety out of a single looped AnimSequence asset.
	*/
	UPROPERTY(EditAnywhere, Category = AnimShake)
	uint32 bRandomAnimSegment : 1;

public:

	/** Time remaining for oscillation shakes. Less than 0.f means shake infinitely. */
	UPROPERTY(transient, BlueprintReadOnly, Category = CameraShake)
	float OscillatorTimeRemaining;

public:

	// Blueprint API

	/** Called when the shake starts playing */
	UFUNCTION(BlueprintImplementableEvent, Category = CameraShake)
	UE_API void ReceivePlayShake(float Scale);

	/** Called every tick to let the shake modify the point of view */
	UFUNCTION(BlueprintImplementableEvent, Category = CameraShake)
	UE_API void BlueprintUpdateCameraShake(float DeltaTime, float Alpha, const FMinimalViewInfo& POV, FMinimalViewInfo& ModifiedPOV);

	/** Called to allow a shake to decide when it's finished playing. */
	UFUNCTION(BlueprintNativeEvent, Category = CameraShake)
	UE_API bool ReceiveIsFinished() const;

	/**
	 * Called when the shake is explicitly stopped.
	 * @param bImmediatly		If true, shake stops right away regardless of blend out settings. If false, shake may blend out according to its settings.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = CameraShake)
	UE_API void ReceiveStopShake(bool bImmediately);

public:

	/**
	 * Backwards compatible method used by core BP redirectors. This is needed because the return value is specifically a legacy camera shake,
	 * which some BP logic often uses directly to set oscillator/anim properties.
	 */
	UFUNCTION(BlueprintCallable, Category = "Camera Shakes")
	static UE_API ULegacyCameraShake* StartLegacyCameraShake(APlayerCameraManager* PlayerCameraManager, TSubclassOf<ULegacyCameraShake> ShakeClass, float Scale = 1.f, ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal, FRotator UserPlaySpaceRot = FRotator::ZeroRotator);

	/**
	 * Backwards compatible method used by core BP redirectors. This is needed because the return value is specifically a legacy camera shake,
	 * which some BP logic often uses directly to set oscillator/anim properties.
	 */
	UFUNCTION(BlueprintCallable, Category = "Camera Shakes")
	static UE_API ULegacyCameraShake* StartLegacyCameraShakeFromSource(APlayerCameraManager* PlayerCameraManager, TSubclassOf<ULegacyCameraShake> ShakeClass, UCameraShakeSourceComponent* SourceComponent, float Scale = 1.f, ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal, FRotator UserPlaySpaceRot = FRotator::ZeroRotator);

public:

	/** Returns true if this camera shake will loop forever */
	UE_API bool IsLooping() const;

private:

	UE_API void DoStartShake(const FCameraShakePatternStartParams& Params);
	UE_API void DoUpdateShake(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult);
	UE_API void DoScrubShake(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult);
	UE_API void DoStopShake(bool bImmediately);
	UE_API bool DoGetIsFinished() const;
	UE_API void DoTeardownShake();

protected:

	/** Actual oscillation duration. */
	float ActualOscillationDuration;

	/** Current location sinusoidal offset. */
	FVector LocSinOffset;

	/** Current rotational sinusoidal offset. */
	FVector RotSinOffset;

	/** Current FOV sinusoidal offset. */
	float FOVSinOffset;

	/** Initial offset (could have been assigned at random). */
	FVector InitialLocSinOffset;

	/** Initial offset (could have been assigned at random). */
	FVector InitialRotSinOffset;

	/** Initial offset (could have been assigned at random). */
	float InitialFOVSinOffset;

	/** Sequence shake pattern for when using a sequence instead of a camera anim */
	UPROPERTY(Instanced)
	TObjectPtr<class USequenceCameraShakePattern> SequenceShakePattern;

private:

	float CurrentBlendInTime;
	float CurrentBlendOutTime;
	bool bBlendingIn : 1;
	bool bBlendingOut : 1;

	friend class ULegacyCameraShakePattern;
};

/**
 * Shake pattern for the ULegacyCameraShake class.
 *
 * It doesn't do anything because, for backwards compatibility reasons, all the data
 * was left on the shake class itself... so this pattern delegates everything back
 * to the owner shake.
 */
UCLASS(MinimalAPI, HideDropdown)
class ULegacyCameraShakePattern : public UCameraShakePattern
{
	GENERATED_BODY()

private:

	// UCameraShakePattern interface
	UE_API virtual void GetShakePatternInfoImpl(FCameraShakeInfo& OutInfo) const override;
	UE_API virtual void StartShakePatternImpl(const FCameraShakePatternStartParams& Params) override;
	UE_API virtual void UpdateShakePatternImpl(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult) override;
	UE_API virtual void ScrubShakePatternImpl(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult) override;
	UE_API virtual bool IsFinishedImpl() const override;
	UE_API virtual void StopShakePatternImpl(const FCameraShakePatternStopParams& Params) override;
	UE_API virtual void TeardownShakePatternImpl() override;
};

/**
 * Blueprint function library for autocasting from a base camera shake to a legacy camera shake.
 * This prevents breaking Blueprints now that APlayerCameraManager::StartCameraShake returns the base class.
 */
UCLASS(MinimalAPI)
class ULegacyCameraShakeFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category = "Camera Shakes", meta = (BlueprintAutocast))
	static ULegacyCameraShake* Conv_LegacyCameraShake(UCameraShakeBase* CameraShake)
	{
		return CastChecked<ULegacyCameraShake>(CameraShake, ECastCheckedType::NullAllowed);
	}
};

#undef UE_API
