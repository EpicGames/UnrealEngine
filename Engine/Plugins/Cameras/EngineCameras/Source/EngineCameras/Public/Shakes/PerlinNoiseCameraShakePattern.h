// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimpleCameraShakePattern.h"
#include "PerlinNoiseCameraShakePattern.generated.h"

#define UE_API ENGINECAMERAS_API

/** A perlin noise shaker for a single number. */
USTRUCT(BlueprintType)
struct FPerlinNoiseShaker
{
	GENERATED_BODY()

	/** Amplitude of the perlin noise. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PerlinNoise)
	float Amplitude;

	/** Frequency of the sinusoidal oscillation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PerlinNoise)
	float Frequency;

	/** Creates a new perlin noise shaker. */
	FPerlinNoiseShaker()
		: Amplitude(1.f)
		, Frequency(1.f)
	{}

	/** Advances the shake time and returns the current value */
	UE_API float Update(float DeltaTime, float AmplitudeMultiplier, float FrequencyMultiplier, float& InOutCurrentOffset) const;
};

/**
 * A camera shake that uses Perlin noise to shake the camera.
 */
UCLASS(MinimalAPI, meta=(AutoExpandCategories="Location,Rotation,FOV,Timing"))
class UPerlinNoiseCameraShakePattern : public USimpleCameraShakePattern
{
public:

	GENERATED_BODY()

	UE_API UPerlinNoiseCameraShakePattern(const FObjectInitializer& ObjInit);

public:

	/** Amplitude multiplier for all location shake */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Location)
	float LocationAmplitudeMultiplier = 1.f;

	/** Frequency multiplier for all location shake */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Location)
	float LocationFrequencyMultiplier = 1.f;

	/** Shake in the X axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Location)
	FPerlinNoiseShaker X;

	/** Shake in the Y axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Location)
	FPerlinNoiseShaker Y;

	/** Shake in the Z axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Location)
	FPerlinNoiseShaker Z;

	/** Amplitude multiplier for all rotation shake */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotation)
	float RotationAmplitudeMultiplier = 1.f;

	/** Frequency multiplier for all rotation shake */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotation)
	float RotationFrequencyMultiplier = 1.f;

	/** Pitch shake. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotation)
	FPerlinNoiseShaker Pitch;

	/** Yaw shake. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotation)
	FPerlinNoiseShaker Yaw;

	/** Roll shake. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotation)
	FPerlinNoiseShaker Roll;

	/** FOV shake. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FOV)
	FPerlinNoiseShaker FOV;

private:

	// UCameraShakePattern interface
	UE_API virtual void StartShakePatternImpl(const FCameraShakePatternStartParams& Params) override;
	UE_API virtual void UpdateShakePatternImpl(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult) override;
	UE_API virtual void ScrubShakePatternImpl(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult) override;

	void UpdatePerlinNoise(float DeltaTime, FCameraShakePatternUpdateResult& OutResult);

private:

	/** Initial perlin noise offset for location oscillation. */
	FVector3f InitialLocationOffset;
	/** Current perlin noise offset for location oscillation. */
	FVector3f CurrentLocationOffset;

	/** Initial perlin noise offset for rotation oscillation. */
	FVector3f InitialRotationOffset;
	/** Current perlin noise offset for rotation oscillation. */
	FVector3f CurrentRotationOffset;

	/** Initial perlin noise offset for FOV oscillation */
	float InitialFOVOffset;
	/** Current perlin noise offset for FOV oscillation */
	float CurrentFOVOffset;
};

#undef UE_API
