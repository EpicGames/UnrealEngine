// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraShakeBase.h"
#include "SimpleCameraShakePattern.generated.h"

#define UE_API ENGINECAMERAS_API

/**
 * A base class for a simple camera shake.
 */
UCLASS(MinimalAPI, Abstract)
class USimpleCameraShakePattern : public UCameraShakePattern
{
public:

	GENERATED_BODY()

	USimpleCameraShakePattern(const FObjectInitializer& ObjInit) : Super(ObjInit) {}

public:

	/** Duration in seconds of this shake. Zero or less means infinite. */
	UPROPERTY(EditAnywhere, Category=Timing)
	float Duration = 1.f;

	/** Blend-in time for this shake. Zero or less means no blend-in. */
	UPROPERTY(EditAnywhere, Category=Timing)
	float BlendInTime = 0.2f;

	/** Blend-out time for this shake. Zero or less means no blend-out. */
	UPROPERTY(EditAnywhere, Category=Timing)
	float BlendOutTime = 0.2f;

protected:

	// UCameraShakePattern interface
	UE_API virtual void GetShakePatternInfoImpl(FCameraShakeInfo& OutInfo) const override;
	UE_API virtual void StartShakePatternImpl(const FCameraShakePatternStartParams& Params) override;
	UE_API virtual bool IsFinishedImpl() const override;
	UE_API virtual void StopShakePatternImpl(const FCameraShakePatternStopParams& Params) override;
	UE_API virtual void TeardownShakePatternImpl()  override;

protected:

	/** The ongoing state for this shake */
	FCameraShakeState State;
};

#undef UE_API
