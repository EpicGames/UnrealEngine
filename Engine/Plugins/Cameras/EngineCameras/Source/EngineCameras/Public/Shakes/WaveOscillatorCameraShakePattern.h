// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimpleCameraShakePattern.h"
#include "WaveOscillatorCameraShakePattern.generated.h"

#define UE_API ENGINECAMERAS_API

/** Shake start offset parameter. */
UENUM()
enum class EInitialWaveOscillatorOffsetType : uint8
{
	/** Start with random offset (default). */
	Random,
	/** Start with zero offset. */
	Zero
};

/** A wave oscillator for a single number. */
USTRUCT(BlueprintType)
struct FWaveOscillator
{
	GENERATED_BODY()

	/** Amplitude of the sinusoidal oscillation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Oscillation)
	float Amplitude;

	/** Frequency of the sinusoidal oscillation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Oscillation)
	float Frequency;

	/** Defines how to begin the oscillation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Oscillation)
	EInitialWaveOscillatorOffsetType InitialOffsetType;
	
	/** Creates a new wave oscillator. */
	FWaveOscillator()
		: Amplitude(1.f)
		, Frequency(1.f)
		, InitialOffsetType(EInitialWaveOscillatorOffsetType::Random)
	{}

	/** Sets the initial offset and returns the initial value of the oscillator */
	UE_API float Initialize(float& OutInitialOffset) const;

	/** Advances the oscillation time and returns the current value */
	UE_API float Update(float DeltaTime, float AmplitudeMultiplier, float FrequencyMultiplier, float& InOutCurrentOffset) const;
};

/**
 * A camera shake that uses oscillations to move the camera.
 */
UCLASS(MinimalAPI, meta=(AutoExpandCategories="Location,Rotation,FOV,Timing"))
class UWaveOscillatorCameraShakePattern : public USimpleCameraShakePattern
{
public:

	GENERATED_BODY()

	UE_API UWaveOscillatorCameraShakePattern(const FObjectInitializer& ObjInit);

public:

	/** Amplitude multiplier for all location oscillation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Location)
	float LocationAmplitudeMultiplier = 1.f;

	/** Frequency multiplier for all location oscillation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Location)
	float LocationFrequencyMultiplier = 1.f;

	/** Oscillation in the X axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Location)
	FWaveOscillator X;

	/** Oscillation in the Y axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Location)
	FWaveOscillator Y;

	/** Oscillation in the Z axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Location)
	FWaveOscillator Z;

	/** Amplitude multiplier for all rotation oscillation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotation)
	float RotationAmplitudeMultiplier = 1.f;

	/** Frequency multiplier for all rotation oscillation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotation)
	float RotationFrequencyMultiplier = 1.f;

	/** Pitch oscillation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotation)
	FWaveOscillator Pitch;

	/** Yaw oscillation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotation)
	FWaveOscillator Yaw;

	/** Roll oscillation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotation)
	FWaveOscillator Roll;

	/** FOV oscillation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FOV)
	FWaveOscillator FOV;

private:

	// UCameraShakePattern interface
	UE_API virtual void StartShakePatternImpl(const FCameraShakePatternStartParams& Params) override;
	UE_API virtual void UpdateShakePatternImpl(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult) override;
	UE_API virtual void ScrubShakePatternImpl(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult) override;

	void UpdateOscillators(float DeltaTime, FCameraShakePatternUpdateResult& OutResult);

private:

	/** Initial sinusoidal offset for location oscillation. */
	FVector3f InitialLocationOffset;
	/** Current sinusoidal offset for location oscillation. */
	FVector3f CurrentLocationOffset;

	/** Initial sinusoidal offset for rotation oscillation. */
	FVector3f InitialRotationOffset;
	/** Current sinusoidal offset for rotation oscillation. */
	FVector3f CurrentRotationOffset;

	/** Initial sinusoidal offset for FOV oscillation */
	float InitialFOVOffset;
	/** Current sinusoidal offset for FOV oscillation */
	float CurrentFOVOffset;
};

#undef UE_API
